// Copyright Epic Games, Inc. All Rights Reserved.

#include "Security/WebAPIAuthentication.h"

#include "Async/Future.h"
#include "WebAPIDeveloperSettings.h"
#include "WebAPILog.h"
#include "WebAPISubsystem.h"
#include "WebAPIUtilities.h"
#include "Engine/Engine.h"
#include "Interfaces/IHttpRequest.h"
#include "Serialization/JsonSerializer.h"

UWebAPIOAuthSettings::UWebAPIOAuthSettings()
{
	SchemeName = TEXT("OAuth");
}

bool UWebAPIOAuthSettings::IsValid() const
{
	return !ClientId.IsEmpty() && !ClientSecret.IsEmpty();
}

bool FWebAPIOAuthSchemeHandler::HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings)
{
	check(InSettings);

	const UWebAPIOAuthSettings* AuthSettings = GetAuthSettings(InSettings);	
	check(AuthSettings);

	if(!ensureMsgf(AuthSettings->IsValid(), TEXT("Authentication settings are missing one or more required properties.")))
	{
		return false;
	}

	if(AuthSettings->AccessToken.IsEmpty())
	{
		return false;
	}

	InRequest->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("%s %s"), *AuthSettings->TokenType, *AuthSettings->AccessToken));

	return true;
}

bool FWebAPIOAuthSchemeHandler::HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse, ESPMode::ThreadSafe> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings)
{
	check(InSettings);

	UWebAPIOAuthSettings* AuthSettings = GetAuthSettings(InSettings);	
	check(AuthSettings);
	
	if(!ensureMsgf(AuthSettings->IsValid(), TEXT("Authentication settings are missing one or more required properties.")))
	{
		return false;
	}

	GEngine->GetEngineSubsystem<UWebAPISubsystem>()->MakeHttpRequest(TEXT("POST"), [&, AuthSettings](const TSharedPtr<IHttpRequest>& InRequest)
	{
		FStringFormatNamedArguments UrlArgs;
		for(const TPair<FString, FString>& KVP : AuthSettings->AdditionalRequestQueryParameters)
		{
			UrlArgs.Add(KVP.Key, KVP.Value);
		}
		
		const FString Url = FString::Format(*AuthSettings->AuthenticationServer, UrlArgs);


		FString StringPayload = "grant_type=client_credentials";
		StringPayload.Append("&client_id=" + AuthSettings->ClientId);
		StringPayload.Append("&client_secret=" + AuthSettings->ClientSecret);

		for(const TPair<FString, FString>& KVP : AuthSettings->AdditionalRequestQueryParameters)
		{
			StringPayload.Append(FString::Printf(TEXT("&%s=%s"), *KVP.Key, *KVP.Value));
		}

		InRequest->SetURL(Url);
		InRequest->SetContentAsString(StringPayload);

		TMap<FString, FString> Headers;

		const FString Host = UWebAPIUtilities::GetHostFromUrl(*Url);

		Headers.Add("Host", Host);
		Headers.Add("Content-Type", "application/x-www-form-urlencoded");

		for (const TPair<FString, FString>& Header : Headers)
		{
			InRequest->SetHeader(Header.Key, Header.Value);
		}
	})
	.Next([this, AuthSettings](const TTuple<FHttpResponsePtr, bool>& InResponse)
	{
		// Request failed
		if(!InResponse.Get<bool>())
		{
			return false;
		}

		const FHttpResponsePtr Response = InResponse.Get<FHttpResponsePtr>();

		FString Message;
		FString AccessTkn;
		bool bSuccess = false;

		if (Response.IsValid())
		{
			const int32 ResponseCode = Response->GetResponseCode();
			Message = Response->GetContentAsString();

			if (EHttpResponseCodes::IsOk(ResponseCode))
			{
				const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
				TSharedPtr<FJsonObject> JsonObject;
				if(FJsonSerializer::Deserialize(JsonReader, JsonObject))
				{
					const int32 UnixTimeExpire = JsonObject->GetNumberField(TEXT("expires_on"));

					UE_LOGF(LogWebAPI, Display, "Generate token Response success");
					AuthSettings->TokenType = JsonObject->GetStringField(TEXT("token_type"));
					AuthSettings->AccessToken = JsonObject->GetStringField(TEXT("access_token"));
					AuthSettings->ExpiresOn =  FDateTime::FromUnixTimestamp(UnixTimeExpire);
					AuthSettings->SaveConfig();
					bSuccess = true;
				}
				else
				{
					UE_LOGF(LogWebAPI, Warning, "Deserialize JSON failed");
					UE_LOGF(LogWebAPI, Error, "Authentication failed: Deserialize JSON Response token failed");
					Message = "Deserialize JSON Response token failed:" + Message;
				}
			}
			else
			{
				UE_LOGF(LogWebAPI, Error, "Authentication failed: Response not valid");
				Message = "Response code not valid:" + Message;
			}
		}
		else
		{
			UE_LOGF(LogWebAPI, Error, "Authentication failed: Generate token Response not valid");
			Message = "Response is null";
		}

		if(bSuccess)
		{
			const FString Host = UWebAPIUtilities::GetHostFromUrl(InResponse.Key->GetURL());

			// Auth succeeded, retry any buffered, previous denied requests
			GEngine->GetEngineSubsystem<UWebAPISubsystem>()->RetryRequestsForHost(Host);
		}

		return bSuccess;
	});

	return true;
}

UWebAPIOAuthSettings* FWebAPIOAuthSchemeHandler::GetAuthSettings(const UWebAPIDeveloperSettings* InSettings)
{
	if(!CachedAuthSettings.IsValid() && InSettings)
	{
		UWebAPIOAuthSettings* OAuthSettings = nullptr;
		InSettings->AuthenticationSettings.FindItemByClass<UWebAPIOAuthSettings>(&OAuthSettings);
		check(OAuthSettings);

		CachedAuthSettings = OAuthSettings;	
	}
	
	return CachedAuthSettings.IsValid() ? CachedAuthSettings.Get() : nullptr;
}
