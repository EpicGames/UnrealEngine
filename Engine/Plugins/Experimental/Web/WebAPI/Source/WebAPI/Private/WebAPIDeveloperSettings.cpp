// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDeveloperSettings.h"

#include "Security/WebAPIAuthentication.h"
#include "WebAPITypes.h"


UWebAPIDeveloperSettings::UWebAPIDeveloperSettings()
{
	AuthenticationSettings.Add(CreateDefaultSubobject<UWebAPIAuthenticationSettings, UWebAPIOAuthSettings>(TEXT("OAuth")));
}

FString UWebAPIDeveloperSettings::FormatUrl(const FString& InSubPath) const
{
	const FString URIScheme = GetURI();
	check(!URIScheme.IsEmpty());

	FString HostStr = Host;
	HostStr.RemoveFromEnd(TEXT("/"));

	FString BaseStr = BaseUrl;
	BaseStr = BaseStr.TrimChar(TEXT('/'));
	if(!BaseStr.IsEmpty())
	{
		BaseStr += TEXT("/");
	}

	FString SubPathStr = InSubPath;
	SubPathStr = SubPathStr.TrimChar(TEXT('/'));
	
	return FString::Printf(TEXT("%s://%s/%s%s"), *URIScheme, *HostStr, *BaseStr, *SubPathStr);
}

FString UWebAPIDeveloperSettings::GetURI() const
{
	if(bOverrideScheme && !URISchemeOverride.IsEmpty())
	{
		return URISchemeOverride;
	}

	const FString PreferredScheme = TEXT("https");
	if(URISchemes.Contains(PreferredScheme))
	{
		return PreferredScheme;		
	}

	if(!URISchemes.IsEmpty())
	{
		return URISchemes[0];		
	}

	return TEXT("");
}

TMap<FString, FString> UWebAPIDeveloperSettings::MakeDefaultHeaders(const FName& InVerb) const
{
	TMap<FString, FString> Headers;
	Headers.Add(TEXT("Host"), Host);

	if(InVerb == UE::WebAPI::HttpVerb::NAME_Post)
	{
		Headers.Add(TEXT("Content-Type"), TEXT("application/json"));
	}

	return Headers;
}

const TArray<TSharedPtr<FWebAPIAuthenticationSchemeHandler>>& UWebAPIDeveloperSettings::GetAuthenticationHandlers() const
{
	return AuthenticationHandlers;
}

#if WITH_EDITOR
void UWebAPIDeveloperSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FProperty* Property = PropertyChangedEvent.Property;
	const FName PropertyName = Property->GetFName();
	if(PropertyName == GET_MEMBER_NAME_CHECKED(UWebAPIDeveloperSettings, AuthenticationSettings))
	{
		for(const TObjectPtr<UWebAPIAuthenticationSettings>& Item : AuthenticationSettings)
		{
			Item->SaveConfig();
		}
		SaveConfig();
	}
}
#endif

bool UWebAPIDeveloperSettings::HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse, ESPMode::ThreadSafe> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings)
{
	return false;
}
