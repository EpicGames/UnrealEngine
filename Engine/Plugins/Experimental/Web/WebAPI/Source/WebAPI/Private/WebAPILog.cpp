// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPILog.h"

DEFINE_LOG_CATEGORY(LogWebAPI);
