// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIModule.h"

#define LOCTEXT_NAMESPACE "WebAPI"

void FWebAPIModule::StartupModule()
{

}

void FWebAPIModule::ShutdownModule()
{

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebAPIModule, WebAPI)
