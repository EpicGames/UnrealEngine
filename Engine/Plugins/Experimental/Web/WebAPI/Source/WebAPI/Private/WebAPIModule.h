// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebAPIModule.h"


class FWebAPIModule final
    : public IWebAPIModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
};
