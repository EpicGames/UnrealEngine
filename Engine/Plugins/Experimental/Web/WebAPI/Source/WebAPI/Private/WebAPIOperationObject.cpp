// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOperationObject.h"

#include "Async/Future.h"
#include "HttpModule.h"
#include "WebAPIDeveloperSettings.h"
#include "WebAPILog.h"
#include "WebAPISubsystem.h"
#include "Engine/Engine.h"

UWebAPIOperationObject::~UWebAPIOperationObject()
{
	Reset();
}

void UWebAPIOperationObject::Reset()
{
	if(HttpRequest.IsValid())
	{
		if(!EHttpRequestStatus::IsFinished(HttpRequest->GetStatus()) && Promise.IsValid())
		{
			Promise->SetValue(FRawResponse{nullptr, nullptr, false, true});
		}

		HttpRequest.Reset();
	}
	
	if(Promise.IsValid())
	{
		Promise.Reset();
	}
}

void UWebAPIOperationObject::OnResponse(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) const
{
	Promise->SetValue(FRawResponse{InRequest, InResponse, bInWasSuccessful, false});
	const EHttpResponseCodes::Type ResponseCode = InResponse ? StaticCast<EHttpResponseCodes::Type>(InResponse->GetResponseCode()) : EHttpResponseCodes::Unknown;
	GEngine->GetEngineSubsystem<UWebAPISubsystem>()->HandleHttpResponse(ResponseCode, InResponse, bInWasSuccessful, InSettings);
}

TFuture<UWebAPIOperationObject::FRawResponse> UWebAPIOperationObject::RequestInternal(const FName& InVerb, UWebAPIDeveloperSettings* InSettings, TUniqueFunction<void(const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>)> OnRequestCreated)
{
	check(!InVerb.IsNone());
	check(InSettings);

	Reset();

    FHttpModule& HttpModule = FHttpModule::Get();
	
    HttpRequest = HttpModule.CreateRequest();
    HttpRequest->SetVerb(InVerb.ToString());

	for(const TTuple<FString, FString>& HeaderPair : InSettings->MakeDefaultHeaders(InVerb))
	{
		HttpRequest->SetHeader(HeaderPair.Key, HeaderPair.Value);
	}

	HttpRequest->OnProcessRequestComplete().BindUObject(this, &UWebAPIOperationObject::OnResponse, InSettings);

	// Allows the caller to inject their own Request properties
	OnRequestCreated(HttpRequest.ToSharedRef());

	GEngine->GetEngineSubsystem<UWebAPISubsystem>()->HandleHttpRequest(HttpRequest, InSettings);

	Promise = MakeShared<TPromise<FRawResponse>, ESPMode::ThreadSafe>();
	
	if(!HttpRequest->ProcessRequest())
	{
		UE_LOGF(LogWebAPI, Warning, "Error processing HttpRequest");
	}

    return Promise->GetFuture();
}
