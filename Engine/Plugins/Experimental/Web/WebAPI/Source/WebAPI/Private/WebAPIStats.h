// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("WebAPI"), STATGROUP_WebAPI, STATCAT_Advanced);
