// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPISubsystem.h"

#include "Async/Future.h"
#include "GameDelegates.h"
#include "HttpModule.h"
#include "WebAPIDeveloperSettings.h"
#include "WebAPIOperationObject.h"
#include "WebAPIUtilities.h"
#include "Engine/Engine.h"
#include "Security/WebAPIAuthentication.h"

TAutoConsoleVariable<int32> CVarMaxDeniedHttpRetryCount(
	TEXT("HTTP.MaxDeniedRetryCount"),
	128,
	TEXT("Maximum number of http retries for denied requests."),
	ECVF_Default);

TObjectPtr<UWebAPIOperationObject> FWebAPIPooledOperation::Pop()
{
	if(ensureMsgf(!ItemClass.IsNull(), TEXT("ItemClass %s could not be loaded."), *ItemClass.GetAssetName()))
	{
		// Nothing available, create new
		if(AvailableItems.IsEmpty())
		{
			TObjectPtr<UWebAPIOperationObject> NewItem = NewObject<UWebAPIOperationObject>(GEngine->GetEngineSubsystemBase(UWebAPISubsystem::StaticClass()), ItemClass.LoadSynchronous());
			return ItemsInUse.Add_GetRef(MoveTemp(NewItem));
		}

		// One or more available, so get last
		TObjectPtr<UWebAPIOperationObject> AvailableItem = AvailableItems.Pop(EAllowShrinking::No);

		// Move to in use and return reference 
		return ItemsInUse.Add_GetRef(MoveTemp(AvailableItem));
	}

	return nullptr;
}

bool FWebAPIPooledOperation::Push(const TObjectPtr<UWebAPIOperationObject>& InItem)
{
	check(InItem);
	
#if UE_BUILD_DEBUG
	check(ItemsInUse.Contains(InItem));
#endif

	// Remove from "in use"
	const int32 ItemsRemoved = ItemsInUse.RemoveSwap(InItem, EAllowShrinking::No);

	// Reset to "new" state
	InItem->Reset();

	// Add/return to available
	AvailableItems.Add(InItem);

	return ItemsRemoved > 0;
}

UWebAPISubsystem::UWebAPISubsystem()
{
}

TObjectPtr<UWebAPIOperationObject> UWebAPISubsystem::MakeOperation(const UWebAPIDeveloperSettings* InSettings, const TSubclassOf<UWebAPIOperationObject>& InClass)
{
	check(InClass);

	if(!bUsePooling)
	{
		return NewObject<UWebAPIOperationObject>(this, InClass);		
	}
	
	FName ClassName = InClass->GetFName();
	FWebAPIPooledOperation& NamedOperationPool = OperationPool.FindOrAdd(MoveTemp(ClassName));
	NamedOperationPool.ItemClass = InClass;
	return NamedOperationPool.Pop();
}

void UWebAPISubsystem::ReleaseOperation(const TSubclassOf<UWebAPIOperationObject>& InClass, const TObjectPtr<UWebAPIOperationObject>& InOperation)
{
	check(InClass);

	if(!bUsePooling)
	{
		return;
	}

	FName ClassName = InClass->GetFName();
	FWebAPIPooledOperation& NamedOperationPool = OperationPool.FindOrAdd(MoveTemp(ClassName));
	NamedOperationPool.Push(InOperation);
}

TFuture<TTuple<TSharedPtr<IHttpResponse, ESPMode::ThreadSafe>, bool>> UWebAPISubsystem::MakeHttpRequest(const FString& InVerb, TUniqueFunction<void(const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>)> OnRequestCreated)
{
	check(!InVerb.IsEmpty());

	const TSharedPtr<TPromise<TTuple<FHttpResponsePtr, bool>>, ESPMode::ThreadSafe> Promise = MakeShared<TPromise<TTuple<FHttpResponsePtr, bool>>, ESPMode::ThreadSafe>();

	FHttpModule& HttpModule = FHttpModule::Get();

	const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& HttpRequest = RequestBuffer.Emplace_GetRef(HttpModule.CreateRequest());
	HttpRequest->SetVerb(InVerb);

	HttpRequest->OnProcessRequestComplete().BindLambda(
		[&, Promise](FHttpRequestPtr InRequestPtr, FHttpResponsePtr InResponse, bool bInWasSuccessful)
		{
			Promise->SetValue(MakeTuple(InResponse, bInWasSuccessful));
		});

	// Allows the caller to inject their own Request properties
	OnRequestCreated(HttpRequest);

	return Promise->GetFuture();
}

void UWebAPISubsystem::RetryRequestsForHost(const FString& InHost)
{
	// Move any requests found for this host to the main queue.
	if(const TArray<TSharedRef<IHttpRequest>>* Requests = HostRequestBuffer.Find(InHost))
	{
		RequestBuffer.Append(*Requests);
		HostRequestBuffer.Remove(InHost);		
	}
}

void UWebAPISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FGameDelegates::Get().GetEndPlayMapDelegate().AddUObject(this, &UWebAPISubsystem::OnEndPlayMap);
}

void UWebAPISubsystem::Deinitialize()
{
	FGameDelegates::Get().GetEndPlayMapDelegate().RemoveAll(this);
	
	Super::Deinitialize();
}

ETickableTickType UWebAPISubsystem::GetTickableTickType() const
{
	if (IsTemplate())
	{
		return ETickableTickType::Never;
	}
	return ETickableTickType::Always;
}

void UWebAPISubsystem::Tick(float DeltaTime)
{
	for(auto It = RequestBuffer.CreateIterator(); It; ++It)
	{
		TSharedRef<IHttpRequest>& Request = *It;
		const EHttpRequestStatus::Type RequestStatus = Request->GetStatus();
		if(RequestStatus == EHttpRequestStatus::NotStarted)
		{
			Request->ProcessRequest();
		}
		else if(RequestStatus == EHttpRequestStatus::Failed)
		{
			if (Request->GetFailureReason() == EHttpFailureReason::ConnectionError)
			{
				Request->ProcessRequest();
			}
			else
			{
				// If there's a response that was denied, keep for retry after auth
				if(Request->GetResponse().IsValid())
				{
					if(Request->GetResponse()->GetResponseCode() == EHttpResponseCodes::Denied)
					{
						FString Host = UWebAPIUtilities::GetHostFromUrl(Request->GetURL());

						TArray<TSharedRef<IHttpRequest>>& HostRequests = HostRequestBuffer.FindOrAdd(Host);
						// Prevent filling this buffer due to never being authorized, etc. - discards requests if buffer full
						if(HostRequests.Num() <= CVarMaxDeniedHttpRetryCount->GetInt())
						{
							HostRequests.Add(Request);
							HostRequestBuffer.Add(Host);
						}

						It.RemoveCurrent();

						continue;
					}
				}

				It.RemoveCurrent();
			}
		}
	}
}

TStatId UWebAPISubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UWebAPISubsystem, STATGROUP_Tickables);
}

bool UWebAPISubsystem::HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings)
{
	const TArray<TSharedPtr<FWebAPIAuthenticationSchemeHandler>>& AuthenticationHandlers = InSettings->GetAuthenticationHandlers();
	for(const TSharedPtr<FWebAPIAuthenticationSchemeHandler>& AuthenticationHandler : AuthenticationHandlers)
	{
		// Returns true if handled, so stop checking other handlers
		if(AuthenticationHandler->HandleHttpRequest(InRequest, InSettings))
		{
			return true;
		}
	}

	return false;
}

bool UWebAPISubsystem::HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings)
{
	check(InSettings);

	// Run certain codes through auth
	if(InResponseCode == EHttpResponseCodes::Denied)
	{
		const TArray<TSharedPtr<FWebAPIAuthenticationSchemeHandler>>& AuthenticationHandlers = InSettings->GetAuthenticationHandlers();
		for(const TSharedPtr<FWebAPIAuthenticationSchemeHandler>& AuthenticationHandler : AuthenticationHandlers)
		{
			// Returns true if handled, so stop checking other handlers
			if(AuthenticationHandler->HandleHttpResponse(InResponseCode, InResponse, bInWasSuccessful, InSettings))
			{
				break;
			}
		}
	}

	return InSettings->HandleHttpResponse(InResponseCode, InResponse, bInWasSuccessful, InSettings);
}

void UWebAPISubsystem::OnEndPlayMap()
{
	OperationPool.Empty();
}
