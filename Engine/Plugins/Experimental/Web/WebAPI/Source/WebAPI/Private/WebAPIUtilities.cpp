// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIUtilities.h"
#include "WebAPIMessageResponse.h"

const FText& UWebAPIUtilities::GetResponseMessage(const FWebAPIMessageResponse& MessageResponse)
{
	return MessageResponse.GetMessage();
}

FString UWebAPIUtilities::GetHostFromUrl(const FString& InUrl)
{
	FString SchemeName;
	FParse::SchemeNameFromURI(*InUrl, SchemeName);
	FString Host = InUrl.Replace(*(SchemeName + TEXT("://")), TEXT(""));
		
	int32 DelimiterIdx = -1;
	if(Host.FindChar(TEXT('/'), DelimiterIdx))
	{
		Host = Host.Left(DelimiterIdx);
	}

	return Host;
}
