// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIHttpRequest.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IWebAPIModuleInterface : public IModuleInterface
{
public:
	virtual void StartupModule() override {}
	virtual void ShutdownModule() override {}

	static IWebAPIModuleInterface& Get()
	{
		static const FName ModuleName = "WebAPI";
		return FModuleManager::LoadModuleChecked<IWebAPIModuleInterface>(ModuleName);
	}

	template <typename PayloadType>
	static TSharedRequest<PayloadType> CreateRequest(const FString& InURL, const FString& Verb = TEXT("POST"), const FString& ContentType = TEXT("application/json"));

	template<typename PayloadType>
	static TSharedRequest<PayloadType> BatchRequests(TArray<TSharedRequest<PayloadType>>&& Requests);
};

template <typename PayloadType>
TSharedRequest<PayloadType> IWebAPIModuleInterface::CreateRequest(const FString& InURL, const FString& Verb, const FString& ContentType)
{
	ensure(IsInGameThread());

	TSharedRef<TRequest<PayloadType>> NewRequest = MakeShared<TRequest<PayloadType>>();
	NewRequest->InitializeRequest(InURL, Verb, ContentType);

	return MoveTemp(NewRequest);
}

template <typename PayloadType>
TSharedRequest<PayloadType> IWebAPIModuleInterface::BatchRequests(TArray<TSharedRef<TRequest<PayloadType>>>&& Requests)
{
	ensure(IsInGameThread() && Requests.Num() > 0);

	if (Requests.Num() == 1)
	{
		ensure(Requests[0]->HasPayload());
		return Requests[0];
	}

	using FTRequestPayloadRef = TSharedRef<TRequest<PayloadType>>;
	FTRequestPayloadRef BatchedRequest = TRequest<PayloadType>::CopyHeadersToNewRequest(Requests[0]);
	
	FString BatchedPayload = TEXT("[");	
	for (const FTRequestPayloadRef& Request : Requests)
	{
		ensure(Request->HasPayload());
		ensure(Requests[0] == Request);
		
		BatchedPayload += Request->PayloadToString().GetValue();
		BatchedPayload += TEXT(',');
	}

	// replaces trailing comma with closing bracket and sets the payload
	BatchedPayload[BatchedPayload.Len() - 1] = TEXT(']');
	const int32 Utf8Length = FTCHARToUTF8_Convert::ConvertedLength(*BatchedPayload, BatchedPayload.Len());
	TArray<uint8> Buffer;
	Buffer.SetNumUninitialized(Utf8Length);
	FTCHARToUTF8_Convert::Convert(reinterpret_cast<UTF8CHAR*>(Buffer.GetData()), Buffer.Num(), *BatchedPayload, BatchedPayload.Len());
	
	BatchedRequest->SetPayloadData(Buffer);

	return BatchedRequest;
}
