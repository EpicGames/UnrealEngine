// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIHttpMessageHandlers.h"
#include "Engine/DeveloperSettings.h"
#include "Interfaces/IHttpResponse.h"

#include "WebAPIAuthentication.generated.h"

class UWebAPIOAuthSettings;

// @note: Each will have user supplied settings, and custom handling of auth requests

/**
 * 
 */
UCLASS(Abstract, Config="Engine")
class WEBAPI_API UWebAPIAuthenticationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Public client identifier. */
	UPROPERTY(BlueprintReadOnly, Category = "Security")
	FName SchemeName;
};

/**
 * 
 */
UCLASS(Config="Engine", PerObjectConfig)
class WEBAPI_API UWebAPIOAuthSettings : public UWebAPIAuthenticationSettings
{
	GENERATED_BODY()

public:
	UWebAPIOAuthSettings();

	/** Public client identifier. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	FString ClientId;

	/** Private client secret.  */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	FString ClientSecret;

	/** Token type, ie. Bearer  */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	FString TokenType = TEXT("Bearer");
	
	/** Private token returned by the server.  */
	UPROPERTY(Config, VisibleAnywhere, BlueprintReadOnly, Category = "Security")
	FString AccessToken;

	/** Private token expiration returned by the server.  */
	UPROPERTY(Config, VisibleAnywhere, BlueprintReadOnly, Category = "Security")
	FDateTime ExpiresOn;

	/** Authentication endpoint. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	FString AuthenticationServer;

	/** Additional query parameters to add to auth request. Each key should be present in the Url. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	TMap<FString, FString> AdditionalRequestQueryParameters;

	/** Additional content key/value pairs to add to auth request. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Security")
	TMap<FString, FString> AdditionalRequestBodyParameters;

	/** Returns true if this contains sufficient information to request authentication. */
	bool IsValid() const;
};

class WEBAPI_API FWebAPIAuthenticationSchemeHandler
	: public FWebAPIHttpRequestHandlerInterface
	, public FWebAPIHttpResponseHandlerInterface
{
public:
	virtual ~FWebAPIAuthenticationSchemeHandler() override = default;
	
	virtual bool HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings) override = 0;
	virtual bool HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) override = 0;
};

class WEBAPI_API FWebAPIOAuthSchemeHandler
	: public FWebAPIAuthenticationSchemeHandler
{
public:
	virtual bool HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings) override;
	virtual bool HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) override;

private:
	UWebAPIOAuthSettings* GetAuthSettings(const UWebAPIDeveloperSettings* InSettings);
	
	TWeakObjectPtr<UWebAPIOAuthSettings> CachedAuthSettings;
};
