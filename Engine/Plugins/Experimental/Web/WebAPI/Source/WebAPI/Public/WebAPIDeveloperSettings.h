// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIHttpMessageHandlers.h"
#include "Engine/DeveloperSettings.h"
#include "Interfaces/IHttpResponse.h"

#include "WebAPIDeveloperSettings.generated.h"

class FWebAPIAuthenticationSchemeHandler;
class UWebAPIAuthenticationSettings;

/**
 * The base class of any auto generated WebAPI settings object.
 */
UCLASS(Abstract, Config="Engine", DefaultConfig)
class WEBAPI_API UWebAPIDeveloperSettings
	: public UDeveloperSettings
	, public FWebAPIHttpResponseHandlerInterface
{
	GENERATED_BODY()
	
public:
	UWebAPIDeveloperSettings();
	
	/** The default host address to access this API. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString Host;

	/** The Url path relative to the host address, ie. "/V1". */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString BaseUrl;

	/** The UserAgent to encode in Http request headers. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString UserAgent = TEXT("X-UnrealEngine-Agent");

	/** The date-time format this API uses to encode/decode from string. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString DateTimeFormat;

	/** Whether to override the URI scheme. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (InlineEditConditionToggle))
	bool bOverrideScheme = false;
	
	/** User specified Uniform Resource Identifier scheme. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverrideScheme"))
	FString URISchemeOverride = TEXT("http");
	
	/** Uniform Resource Identifier schemes (ie. https, http). */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FString> URISchemes;

	/** Whether to print requests to the output log, useful for debugging. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Developer")
	bool bLogRequests = false;

	/** Authentication settings per security scheme. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, NoClear, EditFixedSize, Category = "Security", meta = (TitleProperty = "SchemeName"))
	TArray<TObjectPtr<UWebAPIAuthenticationSettings>> AuthenticationSettings;

	/** Returns a fully formatted Url, including the sub-path. */
	FString FormatUrl(const FString& InSubPath) const;

	/** Returns the most secure URI, or the user specified URI if enabled. */
	FString GetURI() const;

	virtual TMap<FString, FString> MakeDefaultHeaders(const FName& InVerb) const;

	virtual const TArray<TSharedPtr<FWebAPIAuthenticationSchemeHandler>>& GetAuthenticationHandlers() const;

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

protected:
	friend class UWebAPISubsystem;

	TArray<TSharedPtr<FWebAPIAuthenticationSchemeHandler>> AuthenticationHandlers;

	/** Called for all responses, providing the opportunity for custom interception. */
	virtual bool HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<class IHttpResponse, ESPMode::ThreadSafe> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) override;
};
