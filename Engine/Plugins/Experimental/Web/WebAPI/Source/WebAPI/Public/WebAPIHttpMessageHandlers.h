// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

class UWebAPIDeveloperSettings;

/** Implementers receive any Http Request for their owning API. */
class FWebAPIHttpRequestHandlerInterface
{
protected:
	virtual ~FWebAPIHttpRequestHandlerInterface() = default;
			
public:
	/** Return true if the request was handled, subsequent handlers won't be called. */
	virtual bool HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings) = 0;
};


/** Implementers receive any Http Response for their owning API. */
class FWebAPIHttpResponseHandlerInterface
{
protected:
	virtual ~FWebAPIHttpResponseHandlerInterface() = default;
			
public:
	/** Return true if the response was handled, subsequent handlers won't be called. */
	virtual bool HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) = 0;
};
