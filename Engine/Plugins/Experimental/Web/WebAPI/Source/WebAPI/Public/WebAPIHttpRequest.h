// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HttpModule.h"
#include "JsonObjectConverter.h"
#include "Async/Async.h"
#include "Templates/ValueOrError.h"

template <typename PayloadType>
class TRequest : public TSharedFromThis<TRequest<PayloadType>>, public FNoncopyable
{
public:
	void InitializeRequest(const FString& Route, const FString& InVerb = TEXT("POST"), const FString& InContentType = TEXT("json"));

	[[nodiscard]] static TSharedRef<TRequest<PayloadType>> CopyHeadersToNewRequest(const TSharedRef<TRequest<PayloadType>>& Original);
	
	void Post() const;
	void Post(PayloadType&& InPayload);
	void Post(PayloadType&& InPayload, TFunction<void(FHttpRequestPtr, FHttpResponsePtr, bool)> CompletionCallback);

	void Get() const;

	[[nodiscard]] TRequest<PayloadType>& AddHeader(TPair<FString, FString> NewHeader) const;
	[[nodiscard]] TRequest<PayloadType>& BindCompletionCallback(TFunction<void(FHttpRequestPtr, FHttpResponsePtr, bool)> CompletionCallback);
	UE_DEPRECATED(5.3, "BindProgressCallback has been deprecated, use BindProgressCallback64 instead")
	[[nodiscard]] TRequest<PayloadType>& BindProgressCallback(TFunction<void(FHttpRequestPtr, int32 /* BytesSent */, int32 /* BytesReceived */)> ProgressCallback);
	[[nodiscard]] TRequest<PayloadType>& BindProgressCallback64(TFunction<void(FHttpRequestPtr, uint64 /* BytesSent */, uint64 /* BytesReceived */)> ProgressCallback);
	[[nodiscard]] TRequest<PayloadType>& BindRetryCallback(TFunction<void(FHttpRequestPtr, FHttpResponsePtr, float /* SecondsToRetry */)> RetryCallback);
	[[nodiscard]] TRequest<PayloadType>& BindHeaderReceivedCallback(TFunction<void(FHttpRequestPtr, const FString& /* HeaderName */, const FString& /* HeaderValue */)> HeaderReceivedCallback);
	TRequest<PayloadType>& SetPayloadData(const PayloadType& InPayloadData);
	TRequest<PayloadType>& SetPayloadData(TArray<uint8>& InPayloadData);

	[[nodiscard]] bool HasPayload() const;

	[[nodiscard]] inline FString GetVerb() const { return InternalRequest->GetVerb(); }
	[[nodiscard]] inline FString GetContentType() const { return InternalRequest->GetContentType(); }
	[[nodiscard]] inline FString GetRoute() const { return InternalRequest->GetURL(); }

	void SetHeaders(const TMap<FString, FString>& Headers) const;
	[[nodiscard]] TMap<FString, FString> GetAllHeaders() const;
	
	void SetRoute(const FString& InURL) const;
	void SetVerb(const FString& InVerb) const;

	TValueOrError<FString, FString> PayloadToString() const;

	bool operator==(const TRequest<PayloadType>& Rhs) const
	{
		return GetVerb() == Rhs.GetVerb() && GetContentType() == Rhs.GetContentType() && GetRoute() == Rhs.GetRoute();
	}

	bool operator!=(const TRequest<PayloadType>& Rhs) const
	{
		return !(*this == Rhs);
	}
	
	void operator()() const
	{
		InternalRequest->ProcessRequest();
	}

private:

	TValueOrError<FString, FString> PayloadToString_Internal(const PayloadType& InContent) const;

	TSharedRef<IHttpRequest> InternalRequest = FHttpModule::Get().CreateRequest();
};

template <typename PayloadType>
TValueOrError<FString, FString> TRequest<PayloadType>::PayloadToString() const
{
	if (!HasPayload())
	{
		return MakeError(TEXT("Payload was invalid. Make sure you set the payload prior to calling"));
	}

	const auto ByteArrayToString = [](const TArray<uint8>* ByteArray, int32 Size)->FString
	{
		// Custom implementation of UnrealString::BytesToString
		// https://issues.unrealengine.com/issue/UE-33889
		
		FString ConvertedString;
		ConvertedString.Reserve(Size);
		const uint8* RawData = ByteArray->GetData();
		
		while (Size > 0)
		{
			Size--;

			const int16 Value = *RawData;
			ConvertedString += static_cast<TCHAR>(Value);

			++RawData;
		}

		return ConvertedString;
	};
	
	return MakeValue(ByteArrayToString(&InternalRequest->GetContent(), InternalRequest->GetContent().Num()));
}

template <typename PayloadType>
void TRequest<PayloadType>::SetHeaders(const TMap<FString, FString>& Headers) const
{
	for (const TPair<FString, FString>& Header : Headers)
	{
		InternalRequest->SetHeader(Header.Key, Header.Value);
	}
}

template <typename PayloadType>
TMap<FString, FString> TRequest<PayloadType>::GetAllHeaders() const
{
	TMap<FString, FString> OutMap;
	const TArray<FString> Headers = InternalRequest->GetAllHeaders();
	OutMap.Reserve(Headers.Num());
	for (const FString& Header : Headers)
	{
		FString* Key = nullptr;
		FString* Value = nullptr;
		Header.Split(TEXT(":"), Key, Value);
		
		check(Key);
		check(Value);

		if(Key && Value)
		{
			OutMap.Emplace(MoveTemp(*Key), MoveTemp(*Value));
		}
	}

	return OutMap;
}

template <typename PayloadType>
void TRequest<PayloadType>::SetRoute(const FString& InURL) const
{
	InternalRequest->SetURL(InURL);
}

template <typename PayloadType>
void TRequest<PayloadType>::SetVerb(const FString& InVerb) const
{
	InternalRequest->SetVerb(InVerb);
}

template <typename PayloadType>
void TRequest<PayloadType>::InitializeRequest(const FString& Route, const FString& InVerb, const FString& InContentType)
{
	ensure(!Route.IsEmpty());

	SetVerb(InVerb);
	SetRoute(Route);
	InternalRequest->SetHeader(TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent"));
	InternalRequest->SetHeader(TEXT("Content-Type"), InContentType);
	InternalRequest->SetHeader(TEXT("Accept"), InContentType);
	InternalRequest->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));
}

template <typename PayloadType>
TSharedRef<TRequest<PayloadType>> TRequest<PayloadType>::CopyHeadersToNewRequest(const TSharedRef<TRequest<PayloadType>>& Original)
{
	TSharedRef<TRequest<PayloadType>> NewRequest = MakeShared<TRequest<PayloadType>>();
	NewRequest->SetHeaders(Original->GetAllHeaders());
	NewRequest->SetVerb(Original->GetVerb());
	NewRequest->SetRoute(Original->GetRoute());
	return MoveTemp(NewRequest);
}

template <typename PayloadType>
void TRequest<PayloadType>::Post() const
{
	InternalRequest->SetVerb(TEXT("POST"));
	InternalRequest->ProcessRequest();
}

template <typename PayloadType>
void TRequest<PayloadType>::Post(PayloadType&& InPayload)
{
	InternalRequest->SetVerb(TEXT("POST"));
	SetPayloadData(MoveTemp(InPayload))();
	InternalRequest->ProcessRequest();
}

template <typename PayloadType>
void TRequest<PayloadType>::Post(PayloadType&& InPayload,
	TFunction<void(FHttpRequestPtr, FHttpResponsePtr, bool)> CompletionCallback)
{
	InternalRequest->SetVerb(TEXT("POST"));
	InternalRequest->OnProcessRequestComplete().BindLambda(CompletionCallback);
	SetPayloadData(MoveTemp(InPayload))();
	InternalRequest->ProcessRequest();
}

template <typename PayloadType>
void TRequest<PayloadType>::Get() const
{
	InternalRequest->SetVerb(TEXT("GET"));
	InternalRequest->ProcessRequest();
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::AddHeader(TPair<FString, FString> NewHeader) const
{
	InternalRequest->SetHeader(NewHeader.Key, NewHeader.Value);
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::BindCompletionCallback(TFunction<void(FHttpRequestPtr, FHttpResponsePtr, bool)> CompletionCallback)
{
	InternalRequest->OnProcessRequestComplete().BindLambda(MoveTemp(CompletionCallback));
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::BindProgressCallback64(TFunction<void(FHttpRequestPtr, uint64, uint64)> ProgressCallback)
{
	InternalRequest->OnRequestProgress64().BindLambda(MoveTemp(ProgressCallback));
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::BindRetryCallback(TFunction<void(FHttpRequestPtr, FHttpResponsePtr, float)> RetryCallback)
{
	InternalRequest->OnRequestWillRetry().BindLambda(MoveTemp(RetryCallback));
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::BindHeaderReceivedCallback(TFunction<void(FHttpRequestPtr, const FString&, const FString&)> HeaderReceivedCallback)
{
	InternalRequest->OnHeaderReceived().BindLambda(MoveTemp(HeaderReceivedCallback));
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::SetPayloadData(const PayloadType& InPayloadData)
{
	static_assert(TModels_V<CStaticStructProvider, PayloadType>, "PayloadType must be a UStruct");

	FString PayloadAsString = PayloadToString_Internal(InPayloadData).GetValue();
	PayloadAsString.Append(TEXT("]")).InsertAt(0, TEXT("["));
	InternalRequest->SetContentAsString(PayloadAsString);
	
	return *this;
}

template <typename PayloadType>
TRequest<PayloadType>& TRequest<PayloadType>::SetPayloadData(TArray<uint8>& InPayloadData)
{
	InternalRequest->SetContent(MoveTemp(InPayloadData));
	return *this;
}

template <typename PayloadType>
bool TRequest<PayloadType>::HasPayload() const
{
	return InternalRequest->GetContent().Num() > 0;
}

template <typename PayloadType>
TValueOrError<FString, FString> TRequest<PayloadType>::PayloadToString_Internal(const PayloadType& InContent) const
{
	static_assert(TModels_V<CStaticStructProvider, PayloadType>, "PayloadType must be a UStruct");

	FString StringifiedContent;
	if (!FJsonObjectConverter::UStructToJsonObjectString(InContent, StringifiedContent))
	{
		return MakeError(TEXT("Failed to serialize Content. Ensure properties are marked with UPROPERTY() macro"));
	}

	return MakeValue(StringifiedContent);
}

template<typename T>
using TSharedRequest = TSharedRef<TRequest<T>>;