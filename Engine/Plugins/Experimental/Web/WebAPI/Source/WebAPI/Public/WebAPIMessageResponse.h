// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIMessageResponse.generated.h"

/**
 * 
 */
USTRUCT(BlueprintType)
struct WEBAPI_API FWebAPIMessageResponse
{
	GENERATED_BODY()

	virtual ~FWebAPIMessageResponse() = default;

	virtual const FText& GetMessage() const;
};

inline const FText& FWebAPIMessageResponse::GetMessage() const
{
	return FText::GetEmpty();
}
