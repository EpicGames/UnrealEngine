// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"
#include "Interfaces/IHttpRequest.h"
#include "UObject/Object.h"

#include "WebAPIOperationObject.generated.h"

class IHttpRequest;
class UWebAPIDeveloperSettings;

/** Baseclass for an asynchronous Http request/response operation. */
UCLASS(Abstract)
class WEBAPI_API UWebAPIOperationObject : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UWebAPIOperationObject() override;
	
	/** Can contain a response or status message. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Response")
	FText Message;

	/** Reset to initial state. */
	virtual void Reset();

protected:
	struct FRawResponse
	{
		FRawResponse(FHttpRequestPtr InRequest,
			FHttpResponsePtr InResponse,
			bool bInWasSuccessful,
			bool bInWasCancelled = false)
			: Request(InRequest)
			, Response(InResponse)
			, bWasSuccessful(bInWasSuccessful)
			, bWasCancelled(bInWasCancelled)
		{
		}

		FRawResponse(const FRawResponse& Other) = default;
		
		FRawResponse(FRawResponse&& Other) noexcept
			: bWasSuccessful(Other.bWasSuccessful)
			, bWasCancelled(Other.bWasCancelled)
		{
			if(Other.Request.IsValid())
			{
				Request = MoveTemp(Other.Request);
			}

			if(Other.Response.IsValid())
			{
				Response = MoveTemp(Other.Response);
			}
		}
		
		FHttpRequestPtr Request = nullptr;
		FHttpResponsePtr Response = nullptr;
		bool bWasSuccessful = false;
		bool bWasCancelled = false;
	};
	
	/** Active HttpRequest. */
	TSharedPtr<IHttpRequest> HttpRequest;

	/** Pending Response Promise. */
	TSharedPtr<TPromise<FRawResponse>, ESPMode::ThreadSafe> Promise;

	void OnResponse(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) const;

	/** Creates and dispatches a Request, then returns the Response. Use OnRequestCreated callback to inject Request properties. */
	TFuture<FRawResponse> RequestInternal(const FName& InVerb, UWebAPIDeveloperSettings* InSettings, TUniqueFunction<void(const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>)> OnRequestCreated);
};
