// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"
#include "Tickable.h"
#include "WebAPIHttpMessageHandlers.h"
#include "Subsystems/EngineSubsystem.h"

#include "WebAPISubsystem.generated.h"

class IHttpResponse;
class UWebAPIOperationObject;
class UWebAPISubsystem;
class UWebAPIDeveloperSettings;

/** Container for an Operation object pool, per operation type (class). */
USTRUCT()
struct FWebAPIPooledOperation
{
	GENERATED_BODY()

	/** Operation Class (also acts as a key). */
	UPROPERTY()
	TSoftClassPtr<UWebAPIOperationObject> ItemClass;

	/** Pool of un-used operations. */
	UPROPERTY()
	TArray<TObjectPtr<UWebAPIOperationObject>> AvailableItems;

	/** Pool of operations currently in use. */
	UPROPERTY()
	TArray<TObjectPtr<UWebAPIOperationObject>> ItemsInUse;

	/** Returns a new or pooled Item. */
	TObjectPtr<UWebAPIOperationObject> Pop();

	/** Returns the item back to the pool. */
	bool Push(const TObjectPtr<UWebAPIOperationObject>& InItem); 
};

/** Common functionality and top-level parent for shared objects. */
UCLASS()
class WEBAPI_API UWebAPISubsystem
	: public UEngineSubsystem
	, public FTickableGameObject 
	, public FWebAPIHttpRequestHandlerInterface
	, public FWebAPIHttpResponseHandlerInterface
{
	GENERATED_BODY()

public:
	UWebAPISubsystem();
	
	/** Get or create pooled operation for the given OperationType. */
	TObjectPtr<UWebAPIOperationObject> MakeOperation(const UWebAPIDeveloperSettings* InSettings, const TSubclassOf<UWebAPIOperationObject>& InClass);

	/** Get or create pooled operation for the given OperationType. */
	template <typename OperationType>
	TObjectPtr<OperationType> MakeOperation(const UWebAPIDeveloperSettings* InSettings);

	/** Returns the provided operation to the pool, making it available for re-use. */
	void ReleaseOperation(const TSubclassOf<UWebAPIOperationObject>& InClass, const TObjectPtr<UWebAPIOperationObject>& InOperation);

	/** Returns the provided operation to the pool, making it available for re-use. */
	template <typename OperationType>
	void ReleaseOperation(const TObjectPtr<OperationType>& InOperation);

	/** Convenience function to create a simple Http request. */
	TFuture<TTuple<TSharedPtr<IHttpResponse, ESPMode::ThreadSafe>, bool>> MakeHttpRequest(const FString& InVerb, TUniqueFunction<void(const TSharedRef<class IHttpRequest, ESPMode::ThreadSafe>)> OnRequestCreated);

	/** Retry any failed and buffered requests for the given host. */
	void RetryRequestsForHost(const FString& InHost);
	
	//~Begin UEngineSubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~End UEngineSubsystem interface
	
	// FTickableGameObject implementation Begin
	virtual UWorld* GetTickableGameObjectWorld() const override { return GetWorld(); }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	// FTickableGameObject implementation End

private:
	friend class UWebAPIOperationObject;
 
	UPROPERTY()
	bool bUsePooling = false;
	
	UPROPERTY(Transient)
	TMap<FName, FWebAPIPooledOperation> OperationPool;

	/** Requests to process. */
	TArray<TSharedRef<IHttpRequest>> RequestBuffer;

	/** Requests retained per host for retry when response code indicates an auth error. */
	TMap<FString, TArray<TSharedRef<IHttpRequest>>> HostRequestBuffer;

	/** Return true if the request was handled, subsequent handlers won't be called. */
	virtual bool HandleHttpRequest(TSharedPtr<IHttpRequest> InRequest, UWebAPIDeveloperSettings* InSettings) override;

	/** Called for all responses, providing the opportunity for custom interception. */
	virtual bool HandleHttpResponse(EHttpResponseCodes::Type InResponseCode, TSharedPtr<IHttpResponse> InResponse, bool bInWasSuccessful, UWebAPIDeveloperSettings* InSettings) override;
	
	void OnEndPlayMap();
};

template <typename OperationType>
TObjectPtr<OperationType> UWebAPISubsystem::MakeOperation(const UWebAPIDeveloperSettings* InSettings)
{
	static_assert(TModels_V<CStaticClassProvider, OperationType>, "OperationType should provide a StaticClass.");
	return Cast<OperationType>(MakeOperation(InSettings, OperationType::StaticClass()));
}

template <typename OperationType>
void UWebAPISubsystem::ReleaseOperation(const TObjectPtr<OperationType>& InOperation)
{
	static_assert(TModels_V<CStaticClassProvider, OperationType>, "OperationType should provide a StaticClass.");
	ReleaseOperation(OperationType::StaticClass(), InOperation);
}
