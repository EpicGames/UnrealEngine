// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::WebAPI
{
	namespace HttpVerb
	{
		static const FName NAME_Post = FName(TEXT("POST"));
		static const FName NAME_Get = FName(TEXT("GET"));
		static const FName NAME_Put = FName(TEXT("PUT"));
		static const FName NAME_Patch = FName(TEXT("PATCH"));
		static const FName NAME_Delete = FName(TEXT("DELETE"));
		static const FName NAME_Head = FName(TEXT("HEAD"));
		static const FName NAME_Options = FName(TEXT("OPTIONS"));
		static const FName NAME_Trace = FName(TEXT("TRACE"));
	}

	namespace MimeType
	{
		/** List of common mime types */
		
		// Web
		static const FName NAME_Html = FName(TEXT("text/html"));
		static const FName NAME_Css = FName(TEXT("text/css"));
		static const FName NAME_Js = FName(TEXT("application/x-javascript"));

		// Video
		static const FName NAME_Avi = FName(TEXT("video/msvideo, video/avi, video/x-msvideo"));
		static const FName NAME_Mpeg = FName(TEXT("video/mpeg"));

		// Image
		static const FName NAME_Bmp = FName(TEXT("image/bmp"));
		static const FName NAME_Gif = FName(TEXT("image/gif"));
		static const FName NAME_Jpg = FName(TEXT("image/jpeg"));
		static const FName NAME_Jpeg = FName(TEXT("image/jpeg"));
		static const FName NAME_Png = FName(TEXT("image/png"));
		static const FName NAME_Svg = FName(TEXT("image/svg+xml"));
		static const FName NAME_Tiff = FName(TEXT("image/tiff"));

		// Audio
		static const FName NAME_Midi = FName(TEXT("audio/x-midi"));
		static const FName NAME_Mp3 = FName(TEXT("audio/mpeg"));
		static const FName NAME_Ogg = FName(TEXT("audio/vorbis, application/ogg"));
		static const FName NAME_Wav = FName(TEXT("audio/wav, audio/x-wav"));

		// Documents
		static const FName NAME_Xml = FName(TEXT("application/xml"));
		static const FName NAME_Txt = FName(TEXT("text/plain"));
		static const FName NAME_Tsv = FName(TEXT("text/tab-separated-values"));
		static const FName NAME_Csv = FName(TEXT("text/csv"));
		static const FName NAME_Json = FName(TEXT("application/json"));
		
		// Compressed
		static const FName NAME_Zip = FName(TEXT("application/zip, application/x-compressed-zip"));

		static const FName NAME_Unknown = FName(TEXT("application/unknown"));
	}
}
