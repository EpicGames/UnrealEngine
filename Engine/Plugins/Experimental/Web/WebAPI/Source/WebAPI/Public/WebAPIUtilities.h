// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIMessageResponse.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "WebAPIUtilities.generated.h"

/**
 * 
 */
UCLASS()
class WEBAPI_API UWebAPIUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Return the message from the provided response. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="WebAPI")
	static const FText& GetResponseMessage(const FWebAPIMessageResponse& MessageResponse);

	/** Return the host (only) from the provided Url. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="WebAPI")
	static FString GetHostFromUrl(const FString& InUrl);
};
