// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebAPI : ModuleRules
{
	public WebAPI(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Allows display of more verbose information including UI tooltips, etc.
		bool bWithWebAPIDebug = true;
		
		// Forcibly disables the flag if not valid for the Target
		bWithWebAPIDebug &= Target.Type == TargetType.Editor || Target.Type == TargetType.Program;
		PublicDefinitions.Add($"WITH_WEBAPI_DEBUG={(bWithWebAPIDebug ? 1 : 0)}");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "CoreUObject",
                "DeveloperSettings",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"HTTP",
				"Slate",
				"SlateCore",
				"Json",
				"JsonUtilities"
			});

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "UnrealEd"
                });
        }
    }
}
