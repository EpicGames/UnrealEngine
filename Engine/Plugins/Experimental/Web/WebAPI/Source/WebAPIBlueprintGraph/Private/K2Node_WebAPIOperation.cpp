// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_WebAPIOperation.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Self.h"
#include "K2Node_TemporaryVariable.h"
#include "KismetCompiler.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "WebAPIBlueprintGraphLog.h"
#include "WebAPIBlueprintGraphUtilities.h"
#include "WebAPIOperationObject.h"
#include "Algo/AllOf.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ObjectTools.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "K2Node_WebAPIAsyncOperation"

namespace UE::WebAPI::Private
{
	// @note: the next calls are from BaseAsyncTask
	
	static bool ValidDataPin(const UEdGraphPin* InPin, EEdGraphPinDirection InDirection)
	{
		const bool bValidDataPin = InPin
			&& !InPin->bOrphanedPin
			&& (InPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec);

		const bool bProperDirection = InPin && (InPin->Direction == InDirection);

		return bValidDataPin && bProperDirection;
	}

	bool CreateDelegateForNewFunction(UEdGraphPin* DelegateInputPin, FName FunctionName, UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
		check(DelegateInputPin && Schema && CurrentNode && SourceGraph && (FunctionName != NAME_None));
		bool bResult = true;

		// WORKAROUND, so we can create delegate from nonexistent function by avoiding check at expanding step
		// instead simply: Schema->TryCreateConnection(AddDelegateNode->GetDelegatePin(), CurrentCENode->FindPinChecked(UK2Node_CustomEvent::DelegateOutputName));
		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(CurrentNode, SourceGraph);
		SelfNode->AllocateDefaultPins();

		UK2Node_CreateDelegate* CreateDelegateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CreateDelegate>(CurrentNode, SourceGraph);
		CreateDelegateNode->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(DelegateInputPin, CreateDelegateNode->GetDelegateOutPin());
		bResult &= Schema->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), CreateDelegateNode->GetObjectInPin());
		CreateDelegateNode->SetFunction(FunctionName);

		return bResult;
	}

	bool CopyEventSignature(UK2Node_CustomEvent* CENode, UFunction* Function, const UEdGraphSchema_K2* Schema)
	{
		check(CENode && Function && Schema);

		bool bResult = true;
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;
			if (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				FEdGraphPinType PinType;
				bResult &= Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);
				bResult &= (nullptr != CENode->CreateUserDefinedPin(Param->GetFName(), PinType, EGPD_Output));
			}
		}
		return bResult;
	}
	
	bool HandleDelegateImplementation(
		FMulticastDelegateProperty* CurrentProperty, const TArray<FOutputPinAndLocalVariable>& VariableOutputs,
		UEdGraphPin* ProxyObjectPin, UEdGraphPin*& InOutLastThenPin, UEdGraphPin*& OutLastActivatedThenPin,
		UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext)
	{
		bool bIsErrorFree = true;
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
		check(CurrentProperty && ProxyObjectPin && InOutLastThenPin && CurrentNode && SourceGraph && Schema);

		UEdGraphPin* PinForCurrentDelegateProperty = CurrentNode->FindPin(CurrentProperty->GetFName());
		if (!PinForCurrentDelegateProperty || (UEdGraphSchema_K2::PC_Exec != PinForCurrentDelegateProperty->PinType.PinCategory))
		{
			FText ErrorMessage = FText::Format(LOCTEXT("WrongDelegateProperty", "WebAPIOperation: Cannot find execution pin for delegate "), FText::FromString(CurrentProperty->GetName()));
			CompilerContext.MessageLog.Error(*ErrorMessage.ToString(), CurrentNode);
			return false;
		}

		UK2Node_CustomEvent* CurrentCustomEventNode = CompilerContext.SpawnIntermediateNode<UK2Node_CustomEvent>(CurrentNode, SourceGraph);
		{
		UK2Node_AddDelegate* AddDelegateNode = CompilerContext.SpawnIntermediateNode<UK2Node_AddDelegate>(CurrentNode, SourceGraph);
		AddDelegateNode->SetFromProperty(CurrentProperty, false, CurrentProperty->GetOwnerClass());
		AddDelegateNode->AllocateDefaultPins();
			bIsErrorFree &= Schema->TryCreateConnection(AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), ProxyObjectPin);
			bIsErrorFree &= Schema->TryCreateConnection(InOutLastThenPin, AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute));
			InOutLastThenPin = AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
			CurrentCustomEventNode->CustomFunctionName = *FString::Printf(TEXT("%s_%s"), *CurrentProperty->GetName(), *CompilerContext.GetGuid(CurrentNode));
			CurrentCustomEventNode->AllocateDefaultPins();

			bIsErrorFree &= CreateDelegateForNewFunction(AddDelegateNode->GetDelegatePin(), CurrentCustomEventNode->GetFunctionName(), CurrentNode, SourceGraph, CompilerContext);
			bIsErrorFree &= CopyEventSignature(CurrentCustomEventNode, AddDelegateNode->GetDelegateSignature(), Schema);
		}

		OutLastActivatedThenPin = CurrentCustomEventNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
		for (const FOutputPinAndLocalVariable& OutputPair : VariableOutputs) // CREATE CHAIN OF ASSIGMENTS
		{
			UEdGraphPin* PinWithData = CurrentCustomEventNode->FindPin(OutputPair.OutputPin->PinName);
			if (PinWithData == nullptr)
			{
				continue;
			}

			UK2Node_AssignmentStatement* AssignNode = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(CurrentNode, SourceGraph);
			AssignNode->AllocateDefaultPins();
			bIsErrorFree &= Schema->TryCreateConnection(OutLastActivatedThenPin, AssignNode->GetExecPin());
			bIsErrorFree &= Schema->TryCreateConnection(OutputPair.TempVar->GetVariablePin(), AssignNode->GetVariablePin());
			AssignNode->NotifyPinConnectionListChanged(AssignNode->GetVariablePin());
			bIsErrorFree &= Schema->TryCreateConnection(AssignNode->GetValuePin(), PinWithData);
			AssignNode->NotifyPinConnectionListChanged(AssignNode->GetValuePin());

			OutLastActivatedThenPin = AssignNode->GetThenPin();
		}

		bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*PinForCurrentDelegateProperty, *OutLastActivatedThenPin).CanSafeConnect();
		return bIsErrorFree;
	}
}

void UK2Node_WebAPIOperation::AllocateDefaultPins()
{
	if(!IsValid())
	{
		return;
	}
	
	InvalidatePinTooltips();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Exec input/output
	if(!GetExecPin())
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	}

	if(!FindThenPin())
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	}

	TArray<const FMulticastDelegateProperty*, TFixedAllocator<2>> DelegateProperties = {
		UE::WebAPI::Operation::GetPositiveOutcomeDelegate(OperationClass),
		UE::WebAPI::Operation::GetNegativeOutcomeDelegate(OperationClass)
	};

	if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction)
	{
		// Exec outputs for both outcomes
		for(const FMulticastDelegateProperty* DelegateProperty : DelegateProperties)
		{
			UEdGraphPin* ExecPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, DelegateProperty->GetFName());
			ExecPin->PinToolTip = DelegateProperty->GetToolTipText().ToString();
			ExecPin->PinFriendlyName = DelegateProperty->GetDisplayNameText();
		}

		// Response pins
		const UFunction* DelegateSignatureFunction = UE::WebAPI::Operation::GetOutcomeDelegateSignatureFunction(OperationClass);
		if (DelegateSignatureFunction)
		{
			for (TFieldIterator<FProperty> ParameterIterator(DelegateSignatureFunction); ParameterIterator && (ParameterIterator->PropertyFlags & CPF_Parm); ++ParameterIterator)
			{
				const FProperty* Parameter = *ParameterIterator;
				const bool bIsFunctionInput = !Parameter->HasAnyPropertyFlags(CPF_OutParm) || Parameter->HasAnyPropertyFlags(CPF_ReferenceParm);
				if (bIsFunctionInput)
				{
					UEdGraphPin* Pin = CreatePin(EGPD_Output, NAME_None, Parameter->GetFName());
					K2Schema->ConvertPropertyToPinType(Parameter, /*out*/ Pin->PinType);

					// Check for a display name override
					const FString& PinDisplayName = Parameter->GetMetaData(FBlueprintMetadata::MD_DisplayName);
					if (!PinDisplayName.IsEmpty())
					{
						Pin->PinFriendlyName = FText::FromString(PinDisplayName);
					}
					// Else cleanup name
					else
					{
						FString PinNameStr = Pin->PinName.ToString();
						UE::WebAPI::Graph::CleanupPinNameInline(PinNameStr);
						Pin->PinFriendlyName = FText::FromString(PinNameStr);
					}

					UK2Node_CallFunction::GeneratePinTooltipFromFunction(*Pin, DelegateSignatureFunction);

					UE::WebAPI::Graph::SplitPin(Pin);
				}
			}
		}
	}

	bool bAllPinsGood = true;
	if (UFunction* Function = GetFactoryFunction())
	{
		TSet<FName> PinsToHide;
		FBlueprintEditorUtils::GetHiddenPinsForFunction(GetGraph(), Function, PinsToHide);

		// Input pins
		for (TFieldIterator<FProperty> ParameterIterator(Function); ParameterIterator && (ParameterIterator->PropertyFlags & CPF_Parm); ++ParameterIterator)
		{
			const FProperty* Parameter = *ParameterIterator;
			const bool bIsFunctionInput = !Parameter->HasAnyPropertyFlags(CPF_OutParm) || Parameter->HasAnyPropertyFlags(CPF_ReferenceParm);
			if (!bIsFunctionInput)
			{
				// skip function output, it's internal node data 
				continue;
			}

			UEdGraphNode::FCreatePinParams PinParams;
			PinParams.bIsReference = Parameter->HasAnyPropertyFlags(CPF_ReferenceParm) && bIsFunctionInput;
			
			UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Parameter->GetFName(), PinParams);
			const bool bPinGood = (Pin && K2Schema->ConvertPropertyToPinType(Parameter, /*out*/ Pin->PinType));

			if (bPinGood)
			{
				// Check for a display name override
				const FString& PinDisplayName = Parameter->GetMetaData(FBlueprintMetadata::MD_DisplayName);
				if (!PinDisplayName.IsEmpty())
				{
					Pin->PinFriendlyName = FText::FromString(PinDisplayName);
				}
				// Else cleanup name
				else
				{
					FString PinNameStr = Pin->PinName.ToString();
					UE::WebAPI::Graph::CleanupPinNameInline(PinNameStr);
					Pin->PinFriendlyName = FText::FromString(PinNameStr);
				}
				
				//Flag pin as read only for const reference property
				Pin->bDefaultValueIsIgnored = Parameter->HasAllPropertyFlags(CPF_ConstParm | CPF_ReferenceParm) && (!Function->HasMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm) || Pin->PinType.IsContainer());

				const bool bAdvancedPin = Parameter->HasAllPropertyFlags(CPF_AdvancedDisplay);
				Pin->bAdvancedView = bAdvancedPin;
				if(bAdvancedPin && (ENodeAdvancedPins::NoPins == AdvancedPinDisplay))
				{
					AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
				}

				FString ParameterValue;
				if (K2Schema->FindFunctionParameterDefaultValue(Function, Parameter, ParameterValue))
				{
					K2Schema->SetPinAutogeneratedDefaultValue(Pin, ParameterValue);
				}
				else
				{
					K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
				}

				if (PinsToHide.Contains(Pin->PinName))
				{
					Pin->bHidden = true;
				}
			}

			UE::WebAPI::Graph::SplitPin(Pin);

			bAllPinsGood = bAllPinsGood && bPinGood;
		}
	}
	
	Super::AllocateDefaultPins();
}

FText UK2Node_WebAPIOperation::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const FText FunctionName = ObjectTools::GetUserFacingFunctionName(GetFactoryFunction());
	FText NamespaceName;
	FText ServiceName;
	
	if (const UFunction* Function = GetFactoryFunction())
	{
		NamespaceName = Function->GetOuterUClass()->GetMetaDataText(TEXT("Namespace"));
		ServiceName = Function->GetOuterUClass()->GetMetaDataText(TEXT("Service"));
	}
	
	if(TitleType == ENodeTitleType::FullTitle)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("FunctionName"), FunctionName);
		Args.Add(TEXT("NamespaceName"), NamespaceName);
		Args.Add(TEXT("ServiceName"), ServiceName);

		if (NamespaceName.IsEmpty() && ServiceName.IsEmpty())
		{
			return FText::Format(LOCTEXT("NodeTitle", "{FunctionName}"), Args);
		}
		else if (NamespaceName.IsEmpty())
		{
			return FText::Format(LOCTEXT("NodeTitle_WithNamespace", "{FunctionName}\n{ServiceName}"), Args);
		}
		else if (ServiceName.IsEmpty())
		{
			return FText::Format(LOCTEXT("NodeTitle_WithService", "{FunctionName}\n{NamespaceName}"), Args);
		}
		else
		{
			return FText::Format(LOCTEXT("NodeTitle_WithNamespaceAndService", "{FunctionName}\n{NamespaceName}: {ServiceName}"), Args);
		}		
	}
	else
	{
		return FunctionName;
	}
}

FText UK2Node_WebAPIOperation::GetTooltipText() const
{
	FText Tooltip;

	UFunction* Function = GetFactoryFunction();
	if (Function == nullptr)
	{
		return FText::Format(LOCTEXT("CallUnknownFunction", "Call unknown function {0}"), FText::FromName(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction ? LatentFunctionName : DelegatedFunctionName));
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		FText BaseTooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function));

		FFormatNamedArguments Args;
		Args.Add(TEXT("DefaultTooltip"), BaseTooltip);

		if (Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
		{
			Args.Add(
				TEXT("ClientString"),
				NSLOCTEXT("K2Node", "ServerFunction", "Authority Only. This function will only execute on the server.")
			);
			// FText::Format() is slow, so we cache this to save on performance
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("WebAPIOperation_SubtitledTooltip", "{DefaultTooltip}\n\n{ClientString}"), Args), this);
		}
		else if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic))
		{
			Args.Add(
				TEXT("ClientString"),
				NSLOCTEXT("K2Node", "ClientFunction", "Cosmetic. This event is only for cosmetic, non-gameplay actions.")
			);
			// FText::Format() is slow, so we cache this to save on performance
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("WebAPIOperation_SubtitledTooltip", "{DefaultTooltip}\n\n{ClientString}"), Args), this);
		} 
		else
		{
			CachedTooltip.SetCachedText(BaseTooltip, this);
		}
	}
	return CachedTooltip;
}

void UK2Node_WebAPIOperation::ReconstructNode()
{
	if(IsValid())
	{
		// Latent type, so remove input delegate pins
		if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction)
		{
			for(UEdGraphPin* Pin : GetRequestDelegatePins())
			{
				RemovePin(Pin);
			}
		}
		// Callback type, so remove (additional) exec pins and response pins
		else
		{
			for(UEdGraphPin* Pin : GetResponseExecPins())
			{
				RemovePin(Pin);
			}
		}
	}

	Super::ReconstructNode();
}

bool UK2Node_WebAPIOperation::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	return UK2Node::IsCompatibleWithGraph(TargetGraph);
}

void UK2Node_WebAPIOperation::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		// No conversion options if in a function graph (must always be callbacks!)
		if(GetSchema()->GetGraphType(GetGraph()) != EGraphType::GT_Function)
		{
			FText MenuEntryTitle = LOCTEXT("MakeCallbackTitle", "Convert to Callback responses");
			FText MenuEntryTooltip = LOCTEXT("MakeCallbackTooltip", "Removes the execution pins and instead adds callbacks.");

			bool bCanToggleAsyncType = true;
			auto CanExecuteAsyncTypeToggle = [](const bool bInCanToggleAsyncType)->bool
			{
				return bInCanToggleAsyncType;
			};

			if (OperationAsyncType == EWebAPIOperationAsyncType::Callback)
			{
				MenuEntryTitle = LOCTEXT("MakeLatentActionTitle", "Convert to Latent Action");
				MenuEntryTooltip = LOCTEXT("MakeLatentActionTooltip", "Adds in branching execution pins so that you can separatly handle the responses.");

				const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(GetSchema());
				check(K2Schema != nullptr);

				bCanToggleAsyncType = K2Schema->DoesGraphSupportImpureFunctions(GetGraph());
				if (!bCanToggleAsyncType)
				{
					MenuEntryTooltip = LOCTEXT("CannotMakeLatentActionTooltip", "This graph does not support latent actions.");
				}
			}

			FToolMenuSection& Section = Menu->AddSection("K2NodeWebAPIOperation", LOCTEXT("AsyncTypeHeader", "Async Type"));
			Section.AddMenuEntry(
				"ToggleAsyncType",
				MenuEntryTitle,
				MenuEntryTooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_WebAPIOperation*>(this), &UK2Node_WebAPIOperation::ToggleAsyncType),
					FCanExecuteAction::CreateStatic(CanExecuteAsyncTypeToggle, bCanToggleAsyncType),
					FIsActionChecked()
				)
			);
		}
	}
}

bool UK2Node_WebAPIOperation::CanPasteHere(const UEdGraph* TargetGraph) const
{
	return Super::CanPasteHere(TargetGraph);
}

void UK2Node_WebAPIOperation::ExpandNode(
	FKismetCompilerContext& CompilerContext,
	UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(SourceGraph && Schema);
	bool bIsErrorFree = true;

	const UFunction* FactoryFunction = GetFactoryFunction();
	if (FactoryFunction == nullptr)
	{
		const FName ProxyFunctionName =
			OperationAsyncType == EWebAPIOperationAsyncType::LatentAction
			? LatentFunctionName
			: DelegatedFunctionName;
			
		const FText ClassName = OperationClass ? FText::FromString(OperationClass->GetName()) : LOCTEXT("MissingClassString", "Unknown Class");
		const FString FormattedMessage = FText::Format(
			LOCTEXT("WebOperationErrorFmt", "WebAPIOperation: Missing function {0} from class {1} for operation @@"),
			FText::FromString(ProxyFunctionName.GetPlainNameString()),
			ClassName
		).ToString();

		CompilerContext.MessageLog.Error(*FormattedMessage, this);
		return;
	}
		
	// Create a call to factory the proxy object
	UK2Node_CallFunction* CallCreateOperationNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallCreateOperationNode->SetFromFunction(FactoryFunction);
	CallCreateOperationNode->AllocateDefaultPins();

	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Execute), *CallCreateOperationNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute)).CanSafeConnect();

	// Input pins
	for (UEdGraphPin* Pin : Pins)
	{
		if (UE::WebAPI::Private::ValidDataPin(Pin, EGPD_Input))
		{
			UEdGraphPin* DestPin = CallCreateOperationNode->FindPin(Pin->PinName); // match function inputs, to pass data to function from CallFunction node
			bIsErrorFree &= DestPin && CompilerContext.MovePinLinksToIntermediate(*Pin, *DestPin).CanSafeConnect();
		}
	}

	{
		UEdGraphPin* const OperationObjectPin = CallCreateOperationNode->GetReturnValuePin();
		check(OperationObjectPin);

		static const FName AsyncTaskProxyName(TEXT("AsyncTaskProxy"));
		UEdGraphPin* OutputAsyncTaskProxy = FindPin(AsyncTaskProxyName);
		bIsErrorFree &= !OutputAsyncTaskProxy || CompilerContext.MovePinLinksToIntermediate(*OutputAsyncTaskProxy, *OperationObjectPin).CanSafeConnect();

		bIsErrorFree &= ExpandDefaultToSelfPin(CompilerContext, SourceGraph, CallCreateOperationNode);		

		// GATHER OUTPUT PARAMETERS AND PAIR THEM WITH LOCAL VARIABLES
		TArray<UE::WebAPI::Private::FOutputPinAndLocalVariable> VariableOutputs;
		bool bPassedFactoryOutputs = false;
		for (UEdGraphPin* CurrentPin : Pins)
		{
			if ((OutputAsyncTaskProxy != CurrentPin) && UE::WebAPI::Private::ValidDataPin(CurrentPin, EGPD_Output))
			{
				if (!bPassedFactoryOutputs)
				{
					UEdGraphPin* DestPin = CallCreateOperationNode->FindPin(CurrentPin->PinName);
					bIsErrorFree &= DestPin && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
				}
				else
				{
					const FEdGraphPinType& PinType = CurrentPin->PinType;
					UK2Node_TemporaryVariable* TempVarOutput = CompilerContext.SpawnInternalVariable(
						this, PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject.Get(), PinType.ContainerType, PinType.PinValueType);
					bIsErrorFree &= TempVarOutput->GetVariablePin() && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *TempVarOutput->GetVariablePin()).CanSafeConnect();
					VariableOutputs.Add(UE::WebAPI::Private::FOutputPinAndLocalVariable(CurrentPin, TempVarOutput));
				}
			}
			else if (!bPassedFactoryOutputs && CurrentPin && CurrentPin->Direction == EGPD_Output)
			{
				// the first exec that isn't the node's then pin is the start of the asyc delegate pins
				// once we hit this point, we've iterated beyond all outputs for the factory function
				bPassedFactoryOutputs = (CurrentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (CurrentPin->PinName != UEdGraphSchema_K2::PN_Then);
			}
		}

		// FOR EACH DELEGATE DEFINE EVENT, CONNECT IT TO DELEGATE AND IMPLEMENT A CHAIN OF ASSIGMENTS
		UEdGraphPin* LastThenPin = CallCreateOperationNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

		UK2Node_CallFunction* IsValidFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		const FName IsValidFuncName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, IsValid);
		IsValidFuncNode->FunctionReference.SetExternalMember(IsValidFuncName, UKismetSystemLibrary::StaticClass());
		IsValidFuncNode->AllocateDefaultPins();
		UEdGraphPin* IsValidInputPin = IsValidFuncNode->FindPinChecked(TEXT("Object"));

		bIsErrorFree &= Schema->TryCreateConnection(OperationObjectPin, IsValidInputPin);

		UK2Node_IfThenElse* ValidateProxyNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
		ValidateProxyNode->AllocateDefaultPins();
		bIsErrorFree &= Schema->TryCreateConnection(IsValidFuncNode->GetReturnValuePin(), ValidateProxyNode->GetConditionPin());

		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ValidateProxyNode->GetExecPin());
		LastThenPin = ValidateProxyNode->GetThenPin();

		if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction)
		{
			bIsErrorFree &= HandleDelegates(VariableOutputs, OperationObjectPin, LastThenPin, SourceGraph, CompilerContext);
		}

		if (CallCreateOperationNode->FindPinChecked(UEdGraphSchema_K2::PN_Then) == LastThenPin)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingDelegateProperties", "WebAPIOperation: Proxy has no delegates defined. @@").ToString(), this);
			return;
		}

		// Move the connections from the original node then pin to the last internal then pin

		UEdGraphPin* OriginalThenPin = FindPin(UEdGraphSchema_K2::PN_Then);

		if (OriginalThenPin)
		{
			bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*OriginalThenPin, *LastThenPin).CanSafeConnect();
		}
		bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*LastThenPin, *ValidateProxyNode->GetElsePin()).CanSafeConnect();
	}

	if (!bIsErrorFree)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InternalConnectionError", "WebAPIOperation: Internal connection error. @@").ToString(), this);
	}

	// Make sure we caught everything
	BreakAllNodeLinks();
}

FName UK2Node_WebAPIOperation::GetCornerIcon() const
{
	// Only return the latent action icon if that's the current async type, otherwise empty
	return OperationAsyncType == EWebAPIOperationAsyncType::LatentAction
		? Super::GetCornerIcon()
		: NAME_None;
}

void UK2Node_WebAPIOperation::PostPlacedNewNode()
{
	// Prevent latent function in function graph
	if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction && GetSchema()->GetGraphType(GetGraph()) == GT_Function)
	{
		OperationAsyncType = EWebAPIOperationAsyncType::Callback;
		ReconstructNode();
	}
	
	Super::PostPlacedNewNode();
}

void UK2Node_WebAPIOperation::PostPasteNode()
{
	const bool bIsFunctionGraph = GetSchema()->GetGraphType(GetGraph()) == GT_Function;
	
	// Latent action pasted in function graph, forcibly convert to callback style
	if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction && bIsFunctionGraph)
	{
		ToggleAsyncType();
	}

	Super::PostPasteNode();
}

void UK2Node_WebAPIOperation::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeFunc(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UFunction> FunctionPtr)
		{
			UK2Node_WebAPIOperation* OperationNode = CastChecked<UK2Node_WebAPIOperation>(NewNode);
			if (FunctionPtr.IsValid())
			{
				const UFunction* Func = FunctionPtr.Get();

				FString LatentFunctionName = Func->GetName();
				FString DelegatedFunctionName = LatentFunctionName;
				DelegatedFunctionName.RemoveFromEnd(TEXT("Async"));				

				OperationNode->OperationClass			= Func->GetOuterUClass();
				OperationNode->LatentFunctionName		= FName(LatentFunctionName);
				OperationNode->DelegatedFunctionName	= FName(DelegatedFunctionName);

				check(OperationNode->CacheOutcomeDelegates());
			}
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterClassFactoryActions<UWebAPIOperationObject>(
		FBlueprintActionDatabaseRegistrar::FMakeFuncSpawnerDelegate::CreateLambda(
			[NodeClass](const UFunction* FactoryFunc) -> UBlueprintNodeSpawner*
			{
				// Skip functions without "Async" suffix - these should explicitly not be shown.
				// switching implementations between latent (async suffix) and callback (no suffix) is automatic.
				if(!FactoryFunc->GetName().EndsWith(TEXT("Async")))
				{
					return nullptr;
				}

				UBlueprintNodeSpawner* NodeSpawner = UBlueprintFunctionNodeSpawner::Create(FactoryFunc);
				check(NodeSpawner != nullptr);

				NodeSpawner->NodeClass = NodeClass;

				const TWeakObjectPtr<UFunction> FunctionPtr = MakeWeakObjectPtr(const_cast<UFunction*>(FactoryFunc));
				NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeFunc, FunctionPtr);

				return NodeSpawner;
			}));
}

void UK2Node_WebAPIOperation::PostReconstructNode()
{
	// Prevent latent function in function graph
	if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction && GetSchema()->GetGraphType(GetGraph()) == GT_Function)
	{
		ToggleAsyncType();
		return;
	}

	Super::PostReconstructNode();
	InvalidatePinTooltips();
}

void UK2Node_WebAPIOperation::SetAsyncType(EWebAPIOperationAsyncType InAsyncType)
{
	if (InAsyncType != OperationAsyncType)
	{
		OperationAsyncType = InAsyncType;

		// Provides opportunity to perform some actions specific to the conversion direction
		if(OperationAsyncType == EWebAPIOperationAsyncType::LatentAction)
		{
			ConvertCallbackToLatent();			
		}
		else
		{
			ConvertLatentToCallback();
		}

		const bool bHasBeenConstructed = (Pins.Num() > 0);
		if (bHasBeenConstructed)
		{
			ReconstructNode();
		}
	}
}

bool UK2Node_WebAPIOperation::IsValid() const
{
	return (LatentFunctionName != NAME_None || DelegatedFunctionName != NAME_None)
		&& OperationClass != nullptr
		&& ((bHasCompilerMessage && ErrorType >= EMessageSeverity::Info) || !bHasCompilerMessage);
}

#if WITH_EDITOR
EDataValidationResult UK2Node_WebAPIOperation::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);
	
	if(OperationClass == nullptr)
	{
		Context.AddError(LOCTEXT("Missing_OperationClass", "OperationClass is invalid or missing"));
		ValidationResult = EDataValidationResult::Invalid;
	}

	return ValidationResult;
}
#endif

const UK2Node_CustomEvent* UK2Node_WebAPIOperation::GetCustomEventForOutcomeDelegate(
	const UK2Node_WebAPIOperation* InNode,
	const FName& InOutcomeName)
{
	check(InNode);

	TArray<UEdGraphPin*> DelegatePins = InNode->GetRequestDelegatePins();
	if(UEdGraphPin** OutcomeDelegatePin = DelegatePins.FindByPredicate(
		[InOutcomeName](const UEdGraphPin* InPin)
	{
			return InPin->PinName.ToString().Contains(InOutcomeName.ToString());
	}))
	{
		// Found pin, now get connected event
		for(const UEdGraphPin* LinkedPin : (*OutcomeDelegatePin)->LinkedTo)
		{
			if(LinkedPin->GetOwningNode()->IsA<UK2Node_CustomEvent>())
			{
				return Cast<UK2Node_CustomEvent>(LinkedPin->GetOwningNode());
			}
		}
	}

	return nullptr;
}

UFunction* UK2Node_WebAPIOperation::GetFactoryFunction() const
{
	if (OperationClass == nullptr)
	{
		UE_LOGF(LogWebAPIBlueprintGraph, Error, "OperationClass null in %ls. Was a class deleted or saved on a non promoted build?", *GetFullName());
		return nullptr;
	}

	const FName ProxyFunctionName =
		OperationAsyncType == EWebAPIOperationAsyncType::LatentAction
		? LatentFunctionName
		: DelegatedFunctionName;
	
	FMemberReference FunctionReference;
	FunctionReference.SetExternalMember(ProxyFunctionName, OperationClass);

	UFunction* FactoryFunction = FunctionReference.ResolveMember<UFunction>(GetBlueprint());
	
	if (FactoryFunction == nullptr)
	{
		FactoryFunction = OperationClass->FindFunctionByName(ProxyFunctionName);
		UE_CLOGF(FactoryFunction == nullptr, LogWebAPIBlueprintGraph, Error, "FactoryFunction %ls null in %ls. Was a class deleted or saved on a non promoted build?", *ProxyFunctionName.ToString(), *GetFullName());
	}

	return FactoryFunction;
}

void UK2Node_WebAPIOperation::GetRedirectPinNames(
	const UEdGraphPin& Pin,
	TArray<FString>& RedirectPinNames) const
{
	Super::GetRedirectPinNames(Pin, RedirectPinNames);
}

bool UK2Node_WebAPIOperation::ExpandDefaultToSelfPin(
	FKismetCompilerContext& InCompilerContext,
	UEdGraph* InSourceGraph,
	UK2Node_CallFunction* InIntermediateProxyNode)
{
	if(InSourceGraph && InIntermediateProxyNode)
	{
		// Connect a self reference pin if there is a TScriptInterface default to self
		if (const UFunction* TargetFunc = InIntermediateProxyNode->GetTargetFunction())
		{
			const FString& MetaData = TargetFunc->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
			if (!MetaData.IsEmpty())
			{
				// Find the default to self value pin
				if (UEdGraphPin* DefaultToSelfPin = InIntermediateProxyNode->FindPinChecked(MetaData, EGPD_Input))
				{
					// If it has no links then spawn a new self node here
					if (DefaultToSelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface && DefaultToSelfPin->LinkedTo.Num() == 0)
					{
						const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();

						UK2Node_Self* SelfNode = InCompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, InSourceGraph);
						SelfNode->AllocateDefaultPins();
						UEdGraphPin* SelfPin = SelfNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
						// Make a connection from this intermediate self pin to here
						return Schema->TryCreateConnection(DefaultToSelfPin, SelfPin);
					}
				}
			}
		}	
	}
	return true;
}

bool UK2Node_WebAPIOperation::HandleDelegates(
	const TArray<UE::WebAPI::Private::FOutputPinAndLocalVariable>& VariableOutputs,
	UEdGraphPin* ProxyObjectPin,
	UEdGraphPin*& InOutLastThenPin,
	UEdGraph* SourceGraph,
	FKismetCompilerContext& CompilerContext)
{
	bool bIsErrorFree = true;
	for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(OperationClass); PropertyIt && bIsErrorFree; ++PropertyIt)
	{
		UEdGraphPin* LastActivatedThenPin = nullptr;
		bIsErrorFree &= UE::WebAPI::Private::HandleDelegateImplementation(*PropertyIt, VariableOutputs, ProxyObjectPin, InOutLastThenPin, LastActivatedThenPin, this, SourceGraph, CompilerContext);
	}
	return bIsErrorFree;
}

bool UK2Node_WebAPIOperation::CacheOutcomeDelegates()
{
	// If either invalid
	if(!PositiveDelegateProperty.IsValid() || !NegativeDelegateProperty.IsValid())
	{
		check(OperationClass);

		static FString PositiveOutcomeNameStr = UE::WebAPI::Operation::PositiveOutcomeName.ToString();
		static FString NegativeOutcomeNameStr = UE::WebAPI::Operation::NegativeOutcomeName.ToString();
		
		for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(OperationClass); PropertyIt; ++PropertyIt)
		{
			if((*PropertyIt)->GetName().Contains(PositiveOutcomeNameStr))
			{
				PositiveDelegateProperty = *PropertyIt;
			}
			else if((*PropertyIt)->GetName().Contains(NegativeOutcomeNameStr))
			{
				NegativeDelegateProperty = *PropertyIt;
			}
		}	
	}

	check(PositiveDelegateProperty.IsValid());
	check(NegativeDelegateProperty.IsValid());

	return true;
}

void UK2Node_WebAPIOperation::InvalidatePinTooltips()
{
	bPinTooltipsValid = false;
}

UEdGraphPin* UK2Node_WebAPIOperation::FindPin(
	const FName& InName,
	const EEdGraphPinDirection& InDirection,
	const FName& InCategory,
	bool bFindPartial) const
{
	return UE::WebAPI::Graph::FindPin(this, InName, InDirection, InCategory, bFindPartial);	
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::FindPins(
	const FString& InName,
	const EEdGraphPinDirection& InDirection,
	bool bOnlySplitPins) const
{
	return UE::WebAPI::Graph::FindPins(this, InName, InDirection, bOnlySplitPins);
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::GetRequestPins() const
{
	TArray<UEdGraphPin*> RequestPins = FindPins(TEXT("Request"), EGPD_Input);
	ensureMsgf(!RequestPins.IsEmpty(), TEXT("The function must contain a parameter with \"Request\" in the name."));	
	return RequestPins;
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::GetRequestDelegatePins() const
{
	return Pins.FilterByPredicate([](const UEdGraphPin* InPin)
	{
		return InPin->Direction == EGPD_Input
			&& InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate;
	});
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::GetResponsePins() const
{
	return UE::WebAPI::Graph::GetResponsePins(this);
}

UEdGraphPin* UK2Node_WebAPIOperation::FindThenPin() const
{
	return FindPin(UEdGraphSchema_K2::PN_Then, EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_K2::PC_Exec);
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::GetResponseExecPins() const
{
	return Pins.FilterByPredicate([](const UEdGraphPin* InPin)
	{
		return InPin->Direction == EGPD_Output
			&& InPin->PinName.ToString().StartsWith(TEXT("On"))
			&& InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec;
	});
}

TArray<UEdGraphPin*> UK2Node_WebAPIOperation::GetErrorResponsePins() const
{
	return UE::WebAPI::Graph::GetErrorResponsePins(this);
}

UK2Node_CustomEvent* UK2Node_WebAPIOperation::MakeCustomEvent(
	const FName& InName,
	const UFunction* InSignature,
	const FVector2D& InPosition)
{
	if(!GetBlueprint() || !GetBlueprint()->SupportsEventGraphs())
	{
		return nullptr;
	}

	UEdGraph* Graph = GetGraph();
	if(GetSchema()->GetGraphType(GetGraph()) != EGraphType::GT_Ubergraph)
	{
		Graph = GetBlueprint()->GetLastEditedUberGraph();
		if(!Graph)
		{
			FKismetEditorUtilities::CreateDefaultEventGraphs(GetBlueprint());
			// Something went wrong, return nullptr...
			if(GetBlueprint()->UbergraphPages.IsEmpty())
			{
				return nullptr;
			}
			// ... otherwise retry after adding event graph
			else
			{
				return MakeCustomEvent(InName, InSignature, InPosition);
			}
		}
	}

	const FString FunctionName = FString::Printf(TEXT("%s_Event"), *InName.ToString());
	UK2Node_CustomEvent* CustomEventNode = UK2Node_CustomEvent::CreateFromFunction(InPosition, Graph, FunctionName, InSignature, false);

	UE::WebAPI::Graph::SplitPins(UE::WebAPI::Graph::GetResponsePins(CustomEventNode));
	UE::WebAPI::Graph::SplitPins(UE::WebAPI::Graph::GetErrorResponsePins(CustomEventNode));

	return CustomEventNode;
}

/** Named references to Pins. */
struct FLatentActionPinMap
{
public:
	explicit FLatentActionPinMap(const UK2Node_WebAPIOperation* InNode)
	{
		check(InNode);

		ExecIn = InNode->GetExecPin();
		RequestParameters = InNode->GetRequestPins();

		ExecOut = InNode->FindThenPin();
		ExecPositive = InNode->FindPin(UE::WebAPI::Operation::PositiveOutcomeName, EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_K2::PC_Exec, true);
		ExecNegative = InNode->FindPin(UE::WebAPI::Operation::NegativeOutcomeName, EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_K2::PC_Exec, true);
		Responses = InNode->GetResponsePins();
		Responses.Append(InNode->GetErrorResponsePins());
	}
	
	// Inputs
	UEdGraphPin* ExecIn;

	// (Non-split pins)
	TArray<UEdGraphPin*> RequestParameters;

	// Outputs
	UEdGraphPin* ExecOut;
	UEdGraphPin* ExecPositive;
	UEdGraphPin* ExecNegative;

	// (Non-split pins)
	TArray<UEdGraphPin*> Responses;

	UEdGraphPin* GetOutcomeExec(int32 InIndex) const
	{
		return InIndex == 0 ? ExecPositive : ExecNegative;
	}

	bool IsValid() const
	{
		return ExecIn
			&& (RequestParameters.IsEmpty() ? true : Algo::AllOf(RequestParameters))
			&& ExecOut
			&& ExecPositive
			&& ExecNegative
			&& (Responses.IsEmpty() ? true : Algo::AllOf(Responses));		
	}
};

/** Named references to Pins for a single custom event. */
struct FOutcomeEventPinMap
{
public:
	FOutcomeEventPinMap() = default;
	explicit FOutcomeEventPinMap(FName InName, const UK2Node_CustomEvent* InNode, const UFunction* InSignature)
	{
		check(InNode);
		check(InSignature);

		OutcomeName = InName;
		ExecOut = UE::WebAPI::Graph::FindPin(InNode, UEdGraphSchema_K2::PN_Then, EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_K2::PC_Exec);
		DelegatePin =  InNode->FindPinChecked(InNode->DelegateOutputName);
		Responses = UE::WebAPI::Graph::GetResponsePins(InNode);
		Responses.Append(UE::WebAPI::Graph::GetErrorResponsePins(InNode));
	}

	FName OutcomeName = NAME_None;
	
	// Outputs
	UEdGraphPin* ExecOut = nullptr;

	UEdGraphPin* DelegatePin = nullptr;

	// (Non-split pins)
	TArray<UEdGraphPin*> Responses = {};

	bool IsValid() const
	{
		return ExecOut
			&& DelegatePin
			&& (Responses.IsEmpty() ? true : Algo::AllOf(Responses));		
	}
};

/** Named references to Pins. */
struct FCallbackActionPinMap
{
public:
	/** This overload gets the event nodes from the connected delegate pins. */
	explicit FCallbackActionPinMap(const UK2Node_WebAPIOperation* InNode)
		: FCallbackActionPinMap(InNode
			, UK2Node_WebAPIOperation::GetCustomEventForOutcomeDelegate(InNode, UE::WebAPI::Operation::PositiveOutcomeName)
			, UK2Node_WebAPIOperation::GetCustomEventForOutcomeDelegate(InNode, UE::WebAPI::Operation::NegativeOutcomeName))
	{
	}
	
	explicit FCallbackActionPinMap(
		const UK2Node_WebAPIOperation* InNode,
		const UK2Node_CustomEvent* InPositiveEvent,
		const UK2Node_CustomEvent* InNegativeEvent)
	{
		check(InNode);
		check(InPositiveEvent);
		check(InNegativeEvent);
		
		ExecIn = InNode->GetExecPin();
		RequestParameters = InNode->GetRequestPins();
		CallbackPositive = InNode->FindPin(UE::WebAPI::Operation::PositiveOutcomeName, EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_K2::PC_Delegate, true);
		CallbackNegative = InNode->FindPin(UE::WebAPI::Operation::NegativeOutcomeName, EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_K2::PC_Delegate, true);

		check(InNode->PositiveDelegateProperty.IsValid());
		DelegateProperty = InNode->PositiveDelegateProperty.Get(); // Choose one, doesn't matter
		
		ExecOut = InNode->FindThenPin();
		
		PositiveEventNodeMap = FOutcomeEventPinMap(UE::WebAPI::Operation::PositiveOutcomeName, InPositiveEvent, DelegateProperty->SignatureFunction);
		NegativeEventNodeMap = FOutcomeEventPinMap(UE::WebAPI::Operation::NegativeOutcomeName, InNegativeEvent, DelegateProperty->SignatureFunction);
	}
	
	// Inputs
	UEdGraphPin* ExecIn;
	
	// (Non-split pins)
	TArray<UEdGraphPin*> RequestParameters;

	// Outcome delegate property (positive and negative outcomes have same signature)
	FMulticastDelegateProperty* DelegateProperty;
	
	UEdGraphPin* CallbackPositive;
	UEdGraphPin* CallbackNegative;

	// Outputs
	UEdGraphPin* ExecOut;

	// Event Nodes
	FOutcomeEventPinMap PositiveEventNodeMap;
	FOutcomeEventPinMap NegativeEventNodeMap;

	UEdGraphPin* GetCallback(int32 InIndex) const
	{
		return InIndex == 0 ? CallbackPositive : CallbackNegative;
	}

	const FOutcomeEventPinMap& GetEventNodeMap(int32 InIndex) const
	{
		return InIndex == 0 ? PositiveEventNodeMap : NegativeEventNodeMap;
	}

	bool IsValid() const
	{
		return ExecIn
			&& (RequestParameters.IsEmpty() ? true : Algo::AllOf(RequestParameters))
			&& CallbackPositive
			&& CallbackNegative
			&& ExecOut;
	}
};

void UK2Node_WebAPIOperation::ConvertLatentToCallback()
{
	CacheOutcomeDelegates();

	const FLatentActionPinMap LatentActionPinMap(this);
	check(LatentActionPinMap.IsValid());

	const static TArray<FName, TFixedAllocator<2>> OutcomeNames = {UE::WebAPI::Operation::PositiveOutcomeName, UE::WebAPI::Operation::NegativeOutcomeName };
	TArray<UEdGraphPin*, TFixedAllocator<2>> LatentActionOutcomePins = { LatentActionPinMap.ExecPositive, LatentActionPinMap.ExecNegative };

	TArray<UK2Node_CustomEvent*, TFixedAllocator<2>> OutcomeEvents = { nullptr, nullptr };

	// @note: First pass requires old pins
	for(int32 Idx = 0; Idx < OutcomeNames.Num(); ++Idx)
	{
		const UEdGraphPin* OutcomeExecPin = LatentActionOutcomePins[Idx];
		FName OutcomeName = OutcomeNames[Idx];

		// Make custom event
		FVector2D OperationNodePosition = { static_cast<double>(NodePosX), static_cast<double>(NodePosY) };
		OperationNodePosition += FVector2D(-200, OutcomeName == UE::WebAPI::Operation::PositiveOutcomeName ? 200 : 400);
		UK2Node_CustomEvent* CustomEvent = MakeCustomEvent(OutcomeName, PositiveDelegateProperty->SignatureFunction, OperationNodePosition);
		OutcomeEvents[Idx] = CustomEvent;
		check(CustomEvent);

		// Wire event exec to linked
		UEdGraphPin* EventExecPin = UE::WebAPI::Graph::FindPin(CustomEvent, UEdGraphSchema_K2::PN_Then, EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_K2::PC_Exec);
		UE::WebAPI::Graph::TransferPinConnections(OutcomeExecPin, EventExecPin);
	}

	// Refresh
	AllocateDefaultPins();

	const FCallbackActionPinMap CallbackActionPinMap(this, OutcomeEvents[0], OutcomeEvents[1]);
	check(CallbackActionPinMap.IsValid());

	// @note: Second pass requires new pins
	for(int32 Idx = 0; Idx < OutcomeNames.Num(); ++Idx)
	{
		UEdGraphPin* CustomEventDelegatePin = CallbackActionPinMap.GetEventNodeMap(Idx).DelegatePin;

		// Exec pin
		UEdGraphPin* OperationDelegatePin = CallbackActionPinMap.GetCallback(Idx);
		OperationDelegatePin->GetSchema()->TryCreateConnection(CustomEventDelegatePin, OperationDelegatePin);

		TArray<UEdGraphPin*> PinsInUse = UE::WebAPI::Graph::FilterPinsByRelated(LatentActionOutcomePins[Idx], LatentActionPinMap.Responses);
		UE::WebAPI::Graph::TransferPins(PinsInUse, CallbackActionPinMap.GetEventNodeMap(Idx).Responses);
	}
	
	// Refresh
	ReconstructNode();
}

void UK2Node_WebAPIOperation::ConvertCallbackToLatent()
{
	CacheOutcomeDelegates();

	const static TArray<FName, TFixedAllocator<2>> OutcomeNames = { UE::WebAPI::Operation::PositiveOutcomeName, UE::WebAPI::Operation::NegativeOutcomeName };

	// @note: overloaded ctor finds events from connected delegate pins
	const FCallbackActionPinMap CallbackActionPinMap(this);
	check(CallbackActionPinMap.IsValid());

	// Refresh
	AllocateDefaultPins();

	const FLatentActionPinMap LatentActionPinMap(this);
	check(LatentActionPinMap.IsValid());

	for(int32 Idx = 0; Idx < OutcomeNames.Num(); ++Idx)
	{
		// Get custom event for this outcome (from previous callback setup)
		const FOutcomeEventPinMap& CustomEventPinMap = CallbackActionPinMap.GetEventNodeMap(Idx);

		// Wire custom event exec to latent exec pin
		const UEdGraphPin* EventExecPin = CustomEventPinMap.ExecOut;
		UE::WebAPI::Graph::TransferPinConnections(EventExecPin, LatentActionPinMap.GetOutcomeExec(Idx));
		
		// Get custom event exec pin and find relevant nodes (
		TArray<UEdGraphPin*> PinsInUse = UE::WebAPI::Graph::FilterPinsByRelated(CustomEventPinMap.ExecOut, CustomEventPinMap.Responses);
		UE::WebAPI::Graph::TransferPins(PinsInUse, LatentActionPinMap.Responses);

		const bool bNothingLinkedToCustomEvent = Algo::AllOf(CustomEventPinMap.Responses,
			[](const UEdGraphPin* InPin)
			{
				return InPin->LinkedTo.IsEmpty();
			});

		// If nothing linked, delete custom event
		if(bNothingLinkedToCustomEvent)
		{
			UEdGraphNode* CustomEventNode = CustomEventPinMap.ExecOut->GetOwningNode();
			CustomEventNode->GetGraph()->RemoveNode(CustomEventNode);			
		}
	}

	// Refresh
	ReconstructNode();
}

void UK2Node_WebAPIOperation::ToggleAsyncType()
{
	const FText TransactionTitle =
		OperationAsyncType == EWebAPIOperationAsyncType::Callback
		? LOCTEXT("ToggleAsyncTypeToLatentAction", "Convert to Latent Action")
		: LOCTEXT("ToggleAsyncTypeToCallback", "Convert to Callback responses");
	const FScopedTransaction Transaction(TransactionTitle);
	Modify();

	SetAsyncType(StaticCast<EWebAPIOperationAsyncType>(FMath::Abs(StaticCast<int8>(OperationAsyncType) - 1)));
}

#undef LOCTEXT_NAMESPACE
