// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node.h"
#include "UObject/WeakFieldPtr.h"

#include "K2Node_WebAPIOperation.generated.h"

class UK2Node_CustomEvent;

class UK2Node_CallFunction;
class UK2Node_TemporaryVariable;
class FBlueprintActionDatabaseRegistrar;

namespace UE::WebAPI::Private
{
	struct FOutputPinAndLocalVariable
	{
		UEdGraphPin* OutputPin;
		UK2Node_TemporaryVariable* TempVar;

		FOutputPinAndLocalVariable(UEdGraphPin* Pin, UK2Node_TemporaryVariable* Var)
			: OutputPin(Pin)
			, TempVar(Var) { }
	};
}


UENUM()
enum class EWebAPIOperationAsyncType : uint8
{
	LatentAction = 0,
	Callback = 1
};

/**
 * 
 */
UCLASS()
class WEBAPIBLUEPRINTGRAPH_API UK2Node_WebAPIOperation
	: public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void ReconstructNode() override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void PostPlacedNewNode() override;
	virtual void PostPasteNode() override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface.

	// UK2Node interface
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FName GetCornerIcon() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void PostReconstructNode() override;
	// End of UK2Node interface

	/** Set the operation call type. If it differs from the current, the node will be re-arranged to suit the new type. */
	void SetAsyncType(EWebAPIOperationAsyncType InAsyncType);

	/** Validity of this node. */
	bool IsValid() const;

#if WITH_EDITOR
	/** Add node validity to data validation messages. */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

protected:
	// Returns the factory function (checked)
	virtual UFunction* GetFactoryFunction() const;

	/** Determines what the possible redirect pin names are **/
	virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const override;

	/**
	 * If a the DefaultToSelf pin exists then it needs an actual connection to get properly casted
	 * during compilation.
	 *
	 * @param InCompilerContext			The current compiler context used during expansion
	 * @param InSourceGraph				The graph to place the expanded self node on
	 * @param InIntermediateProxyNode		The spawned intermediate proxy node that has a DefaultToSelfPin
	 *
	 * @return	True if a successful connection was made to an intermediate node
	 *			or if one was not necessary. False if a connection was failed.
	 */
	bool ExpandDefaultToSelfPin(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph, UK2Node_CallFunction* InIntermediateProxyNode);

	/** Expand out the logic to handle the delegate output pins */
	virtual bool HandleDelegates(
		const TArray<UE::WebAPI::Private::FOutputPinAndLocalVariable>& VariableOutputs, UEdGraphPin* ProxyObjectPin,
		UEdGraphPin*& InOutLastThenPin, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext);

	/** Checks and caches outcome (success, error) delegates, returning true if both resolved. */
	bool CacheOutcomeDelegates();

	/** Invalidates current pin tool tips, so that they will be refreshed before being displayed: */
	void InvalidatePinTooltips();

private:
	friend struct FLatentActionPinMap;
	friend struct FCallbackActionPinMap;
	friend struct FOutcomeEventPinMap;

	/** Find a single pin according to various criteria. */
	UEdGraphPin* FindPin(
		const FName& InName,
		const EEdGraphPinDirection& InDirection = EEdGraphPinDirection::EGPD_MAX,
		const FName& InCategory = NAME_All,
		bool bFindPartial = false) const;

	/** Find multiple pins according to various criteria. */
	TArray<UEdGraphPin*> FindPins(
		const FString& InName,
		const EEdGraphPinDirection& InDirection = EEdGraphPinDirection::EGPD_MAX,
		bool bOnlySplitPins = true) const;

	UEdGraphPin* FindThenPin() const;

	/** Get all input pins that correspond to operation parameters. */
	TArray<UEdGraphPin*> GetRequestPins() const;

	/** Get both input request delegate pins, will return empty if this is Latent. */
	TArray<UEdGraphPin*> GetRequestDelegatePins() const;

	/** Get all (non-exec, non-error) output pins, will return empty if this uses Callbacks. */
	TArray<UEdGraphPin*> GetResponsePins() const;

	/** Get both outcome exec pins, will return empty if this uses Callbacks. */
	TArray<UEdGraphPin*> GetResponseExecPins() const;

	/** Get all error output pins, will return empty if this uses Callbacks. */
	TArray<UEdGraphPin*> GetErrorResponsePins() const;

	/** Make and return a custom event in either the calling graph (if not function), or event graph. */
	class UK2Node_CustomEvent* MakeCustomEvent(const FName& InName,
		const UFunction* InSignature,
		const FVector2D& InPosition = {});

	/** Get or make a custom event node for the supplied outcome.  */
	static const UK2Node_CustomEvent* GetCustomEventForOutcomeDelegate(const UK2Node_WebAPIOperation* InNode, const FName& InOutcomeName);

	/** Called when converting FROM latent TO callback. */
	void ConvertLatentToCallback();

	/** Called when converting FROM callback TO latent. */
	void ConvertCallbackToLatent();

	/** Flips the node's Async Type. */
	void ToggleAsyncType();

	/** The class containing the operation functions, AND is the return type of the latent function. */
	UPROPERTY()
	TObjectPtr<UClass> OperationClass;

	/** The name of the latent function to call to create an operation object. */
	UPROPERTY()
	FName LatentFunctionName;
	
	/** The name of the delegated function to call to create an operation object. */
	UPROPERTY()
	FName DelegatedFunctionName;

	/** Reference to the operations' positive outcome delegate property. */
	TWeakFieldPtr<FMulticastDelegateProperty> PositiveDelegateProperty;

	/** Reference to the operations' negative outcome delegate property. */
	TWeakFieldPtr<FMulticastDelegateProperty> NegativeDelegateProperty;

	/** Latent Action is preferred, but not compatible when used in a function, so allow conversion between the two. */
	UPROPERTY()
	EWebAPIOperationAsyncType OperationAsyncType = EWebAPIOperationAsyncType::LatentAction;

	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;

	/** Flag used to track validity of pin tooltips, when tooltips are invalid they will be refreshed before being displayed */
	mutable bool bPinTooltipsValid;
};
