// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionFilter.h"
#include "K2Node_WebAPIOperation.h"
#include "Modules/ModuleManager.h"

class FWebAPIBlueprintGraphModule final
	: public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// Add actions that are relevant to the bound object from the pin class
		FBlueprintActionFilter WebAPIOperationFilter(FBlueprintActionFilter::BPFILTER_NoFlags);
		WebAPIOperationFilter.PermittedNodeTypes.Add(UK2Node_WebAPIOperation::StaticClass());
		

		FBlueprintActionFilter::AddUnique(WebAPIOperationFilter.TargetClasses, UK2Node_WebAPIOperation::StaticClass());
	}
	
	virtual void ShutdownModule() override { }
};

IMPLEMENT_MODULE(FWebAPIBlueprintGraphModule, WebAPIBlueprintGraph)
