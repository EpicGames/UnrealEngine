// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIBlueprintGraphUtilities.h"

#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "WebAPIOperationObject.h"
#include "Algo/AllOf.h"

namespace UE::WebAPI
{
	namespace Operation
	{
		template <uint32 Index, typename TEnableIf<Index < 2>::Type* = nullptr>
		const FMulticastDelegateProperty* GetOutcomeDelegate(const TSubclassOf<UWebAPIOperationObject>& InOperationClass, FName InOutcomeName)
		{
			check(InOperationClass);

			FCachedOutcome* CachedOutcome = nullptr;

			// If cached found
			if(FCachedOutcome* Found = CachedOutcomeDelegates.Find(InOperationClass->GetFName()))
			{
				//  If valid, just return it
				if(Found->Get<Index>().IsValid())
				{
					return Found->Get<Index>().Get();
				}
				
				// Set the pointer for later use
				CachedOutcome = Found;
			}
			// Otherwise add it as a cache entry
			else
			{
				CachedOutcome = &CachedOutcomeDelegates.Emplace(InOperationClass->GetFName());
			}

			FMulticastDelegateProperty* P = nullptr;
			const FString OutcomeNameStr = InOutcomeName.ToString();
			for (TFieldIterator<FMulticastDelegateProperty> PropertyIterator(InOperationClass); PropertyIterator; ++PropertyIterator)
			{
				if(PropertyIterator->GetName().Contains(OutcomeNameStr))
				{
					P = *PropertyIterator;
					CachedOutcome->Set<Index>(*PropertyIterator);
					return P;
				}
			}

			return nullptr;
		}
		
		const FMulticastDelegateProperty* GetPositiveOutcomeDelegate(const TSubclassOf<UWebAPIOperationObject>& InOperationClass)
		{
			check(InOperationClass);

			return GetOutcomeDelegate<0>(InOperationClass, PositiveOutcomeName);
		}

		const FMulticastDelegateProperty* GetNegativeOutcomeDelegate(const TSubclassOf<UWebAPIOperationObject>& InOperationClass)
		{
			check(InOperationClass);

			return GetOutcomeDelegate<1>(InOperationClass, NegativeOutcomeName);
		}

		UFunction* GetOutcomeDelegateSignatureFunction(const TSubclassOf<UWebAPIOperationObject>& InOperationClass)
		{
			return GetPositiveOutcomeDelegate(InOperationClass)->SignatureFunction;			
		}
	}

	namespace Graph
	{
		// @note: the next 4 calls are from FBlueprintEditor
	
		void CollectExecDownstreamNodes(const UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

			for (UEdGraphPin*& Pin : AllPins)
			{
				if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == K2Schema->PC_Exec)
				{
					for (UEdGraphPin*& Link : Pin->LinkedTo)
					{
						UEdGraphNode* LinkedNode = Cast<UEdGraphNode>(Link->GetOwningNode());
						if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
						{
							CollectedNodes.Add(LinkedNode);
							CollectExecDownstreamNodes( LinkedNode, CollectedNodes );
						}
					}
				}
			}
		}

		void CollectExecUpstreamNodes(const UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

			for (UEdGraphPin*& Pin : AllPins)
			{
				if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory == K2Schema->PC_Exec)
				{
					for (UEdGraphPin*& Link : Pin->LinkedTo)
					{
						UEdGraphNode* LinkedNode = Cast<UEdGraphNode>(Link->GetOwningNode());
						if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
						{
							CollectedNodes.Add(LinkedNode);
							CollectExecUpstreamNodes( LinkedNode, CollectedNodes );
						}
					}
				}
			}
		}

		void CollectPureDownstreamNodes(const UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

			for (UEdGraphPin*& Pin : AllPins)
			{
				if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory != K2Schema->PC_Exec)
				{
					for (UEdGraphPin*& Link : Pin->LinkedTo)
					{
						UK2Node* LinkedNode = Cast<UK2Node>(Link->GetOwningNode());
						if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
						{
							CollectedNodes.Add(LinkedNode);
							if (LinkedNode->IsNodePure())
							{
								CollectPureDownstreamNodes( LinkedNode, CollectedNodes );
							}
						}
					}
				}
			}
		}

		void CollectPureUpstreamNodes(const UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

			for (UEdGraphPin*& Pin : AllPins)
			{
				if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != K2Schema->PC_Exec)
				{
					for (UEdGraphPin*& Link : Pin->LinkedTo)
					{
						UK2Node* LinkedNode = Cast<UK2Node>(Link->GetOwningNode());
						if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
						{
							CollectedNodes.Add(LinkedNode);
							if (LinkedNode->IsNodePure())
							{
								CollectPureUpstreamNodes( LinkedNode, CollectedNodes );
							}
						}
					}
				}
			}
		}
		
		void TransferPinConnections(
			const UEdGraphPin* InSrc,
			UEdGraphPin* InDst)
		{
			TArray<UEdGraphPin*> LinkedPins = InSrc->LinkedTo;
			for(UEdGraphPin* LinkedPin : LinkedPins)
			{
				// Link Dst to LinkedPin
				InDst->GetSchema()->TryCreateConnection(InDst, LinkedPin);
			}
		}

		// For every src pin, get the corresponding dst pin - split if necessary
		void TransferPins(
			const TArray<UEdGraphPin*>& InSrcPins,
			const TArray<UEdGraphPin*>& InDstPins)
		{
			for(UEdGraphPin* SrcPin : InSrcPins)
			{
				if(SrcPin->bHidden)
				{
					continue;
				}
		
				// root = not split
				const bool bIsRootPin = !SrcPin->ParentPin && SrcPin->SubPins.IsEmpty();
				if(bIsRootPin)
				{
					if(UEdGraphPin* const* MatchingDstPin = InDstPins.FindByPredicate(
						[SrcPinName = SrcPin->PinName](const UEdGraphPin* InPin)
						{
							return InPin->PinName == SrcPinName;
						}))
					{
						UE::WebAPI::Graph::TransferPinConnections(SrcPin, *MatchingDstPin);
					}
				}
				else
				{
					TArray Hierarchy = { SrcPin->PinName };

					// Find root pin
					const UEdGraphPin* RootPin = SrcPin->ParentPin;
					while(RootPin != nullptr)
					{
						Hierarchy.Add(RootPin->PinName);
						RootPin = RootPin->ParentPin;
					}

					Algo::Reverse(Hierarchy);

					// Confined to current hierachy/subpins
					TArray<UEdGraphPin*> DstPinsToSearch = InDstPins;
					for(int32 PinIdx = 0; PinIdx < Hierarchy.Num(); ++PinIdx)
					{
						FName PinName = Hierarchy[PinIdx];
				
						if(UEdGraphPin* const* MatchingDstPin = DstPinsToSearch.FindByPredicate(
							[SrcPinName = PinName](const UEdGraphPin* InPin)
							{
								return InPin->PinName == SrcPinName;
							}))
						{
							const bool bIsLastItem = PinIdx == Hierarchy.Num() - 1;
							if(bIsLastItem)
							{
								TransferPinConnections(SrcPin, *MatchingDstPin);
								continue;
							}
					
							if((*MatchingDstPin)->SubPins.IsEmpty())
							{
								(*MatchingDstPin)->GetSchema()->SplitPin(*MatchingDstPin);
						
								// Removes the prefix from the auto-named split pins 
								if ((*MatchingDstPin)->SubPins.Num() > 0)
								{
									for (UEdGraphPin* SubPin : (*MatchingDstPin)->SubPins)
									{
										FString SubPinName = SubPin->PinName.ToString();
										CleanupPinNameInline(SubPinName);

										//SubPin->Modify();			
										SubPin->PinFriendlyName = FText::FromString(SubPinName);
									}
								}
							}

							DstPinsToSearch = (*MatchingDstPin)->SubPins;
						}
					}
				}
			}
		}

		bool SplitPins(const TArray<UEdGraphPin*>& InPins)
		{
			// Compilation errors or stale nodes should be warnings, not errors
			if(!ensureAlways(!InPins.IsEmpty() && Algo::AllOf(InPins, [](const UEdGraphPin* InPin) { return InPin; })))
			{
				return false;
			}

			bool bAllGood = true;
			for(UEdGraphPin* Pin : InPins)
			{
				bAllGood &= SplitPin(Pin);	
			}

			return bAllGood;
		}

		bool SplitPin(UEdGraphPin* InPin)
		{
			bool bWasSplit = false;
			if (const UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
			{
				uint32 PropertyNum = 0;
			
				// Split if not already
				if (InPin->SubPins.Num() == 0 && InPin->GetOwningNode()->CanSplitPin(InPin))
				{
					for (TFieldIterator<const FProperty> PropertyIt(PinStructType, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated); PropertyIt; ++PropertyIt)
					{
						if (PropertyIt->HasAnyPropertyFlags(CPF_Transient))
						{
							continue;
						}

						PropertyNum++;
					}

					if(PropertyNum == 1)
					{
						// Split, result should be single pin (cause it only contained a single property!)
						const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
						K2Schema->SplitPin(InPin);
					}
				}

				// Removes the prefix from the auto-named split pins 
				if (InPin->SubPins.Num() > 0)
				{
					bWasSplit = true;
					for (UEdGraphPin* SubPin : InPin->SubPins)
					{
						FString SubPinName = SubPin->PinName.ToString();
						CleanupPinNameInline(SubPinName);
	
						SubPin->PinFriendlyName = FText::FromString(SubPinName);
					}
				}
			}

			return bWasSplit;
		}

		TArray<UEdGraphPin*> FilterPinsByRelated(
			const UEdGraphPin* InExecutionPin,
			const TArray<UEdGraphPin*>& InPinsToFilter)
		{
			TMap<FName, UEdGraphPin*> NameToPin; // key = pin name
			Algo::Transform(InPinsToFilter, NameToPin,
				[](UEdGraphPin* InPin)
				{
					return TPair<FName, UEdGraphPin*>(InPin->PinName, InPin);
				});
			
			TMap<FName, TArray<UEdGraphPin*>> LinkedPins; // key = pin name
			Algo::Transform(InPinsToFilter, LinkedPins,
				[](const UEdGraphPin* InPin)
				{
					return TPair<FName, TArray<UEdGraphPin*>>(InPin->PinName, InPin->LinkedTo);
				});
			
			TMap<FName, TSet<UEdGraphNode*>> LinkedNodes; // key = pin name
			Algo::Transform(InPinsToFilter, LinkedNodes,
				[](const UEdGraphPin* InPin)
				{
					TSet<UEdGraphNode*> LinkedToNodes;
					for(const UEdGraphPin* LinkedTo : InPin->LinkedTo)
					{
						LinkedToNodes.Add(LinkedTo->GetOwningNode());				
					}
					
					return TPair<FName, TSet<UEdGraphNode*>>(InPin->PinName, LinkedToNodes);
				});

			TFunction<void(const UEdGraphPin*, TArray<UEdGraphNode*>&)> AppendConnectedToExec;
			AppendConnectedToExec = [&AppendConnectedToExec](const UEdGraphPin* InPin, TArray<UEdGraphNode*>& OutNodes)
			{
				for(const UEdGraphPin* LinkedPin : InPin->LinkedTo)
				{
					UEdGraphNode* LinkedNode = LinkedPin->GetOwningNode();
					OutNodes.Add(LinkedNode);

					TArray<UEdGraphPin*> LinkedNodeExecPins = LinkedNode->GetAllPins().FilterByPredicate([](const UEdGraphPin* InLinkedNodePin)
					{
						return InLinkedNodePin->Direction == EGPD_Output
							&& InLinkedNodePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec;
					});

					for(const UEdGraphPin* LinkedNodeExecPin : LinkedNodeExecPins)
					{
						AppendConnectedToExec(LinkedNodeExecPin, OutNodes);
					}
				}
			};

			TArray<UEdGraphNode*> NodesInExecutionChain;
			AppendConnectedToExec(InExecutionPin, NodesInExecutionChain);
			
			TArray<UEdGraphNode*> RelatedNodes;

			TArray<UEdGraphNode*> ImpureNodes = NodesInExecutionChain.FilterByPredicate([](UEdGraphNode* InNode){
				const UK2Node* K2Node = Cast<UK2Node>(InNode);
				if (K2Node)
				{
					return !K2Node->IsNodePure();
				}
				return false;
			});

			TArray<UEdGraphNode*> PureNodes = NodesInExecutionChain.FilterByPredicate([](UEdGraphNode* InNode){
				const UK2Node* K2Node = Cast<UK2Node>(InNode);
				if (K2Node)
				{
					return K2Node->IsNodePure();
				}
				// Treat a node which can't cast to an UK2Node as a pure node (like a document node or a commment node)
				// Make sure all selected nodes are handled
				return true;
			});

			for (UEdGraphNode* ImpureNode : ImpureNodes)
			{
				RelatedNodes.Add(ImpureNode);
				CollectExecDownstreamNodes(ImpureNode, RelatedNodes);
				CollectExecUpstreamNodes(ImpureNode, RelatedNodes);
				CollectPureDownstreamNodes(ImpureNode, RelatedNodes);
				CollectPureUpstreamNodes(ImpureNode, RelatedNodes);
			}

			for (UEdGraphNode* PureNode : PureNodes)
			{
				RelatedNodes.Add(PureNode);
				CollectPureDownstreamNodes(PureNode, RelatedNodes);
				CollectPureUpstreamNodes(PureNode, RelatedNodes);
			}

			TSet<UEdGraphNode*> RelatedNodesSet{RelatedNodes};
			TArray<UEdGraphPin*> UsedPins;

			for(const TPair<FName, TSet<UEdGraphNode*>>& PinNodes : LinkedNodes)
			{
				// If any nodes for this pin are in the related nodes (for the current outcome execution)
				if(!PinNodes.Value.Intersect(RelatedNodesSet).IsEmpty())
				{
					UsedPins.Add(NameToPin[PinNodes.Key]);			
				}
			}

			return UsedPins;
		}

		UEdGraphPin* FindPin(
			const UEdGraphNode* InNode,
			const FName& InName,
			const EEdGraphPinDirection& InDirection,
			const FName& InCategory,
			bool bFindPartial)
		{
			FString NameStr = InName.ToString();
	
			return InNode->FindPinByPredicate([&InName, &InDirection, &InCategory, bFindPartial, &NameStr](const UEdGraphPin* InPin)
			{
				return (bFindPartial ? InPin->PinName.ToString().Contains(NameStr) : InPin->PinName == InName)
					&& (InDirection < EEdGraphPinDirection::EGPD_MAX ? InPin->Direction == InDirection : true)
					&& (InCategory != NAME_All ? InPin->PinType.PinCategory == InCategory : true);
			});
		}

		TArray<UEdGraphPin*> FindPins(
			const UEdGraphNode* InNode,
			const FString& InName,
			const EEdGraphPinDirection& InPinDirection,
			bool bOnlySplitPins)
		{
			return InNode->Pins.FilterByPredicate([&InPinDirection, InName, bOnlySplitPins](const UEdGraphPin* InPin)
			{
				return InPin->Direction == InPinDirection
					&& (bOnlySplitPins ? InPin->ParentPin || InPin->SubPins.IsEmpty() : InPin->SubPins.IsEmpty())
					&& InPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec
					&& InPin->GetName().Contains(InName);
			});
		}

		TArray<UEdGraphPin*> GetResponsePins(const UEdGraphNode* InNode)
		{
			TArray<UEdGraphPin*> ResponsePins = FindPins(InNode, TEXT("Response"), EGPD_Output);
			ensureMsgf(!ResponsePins.IsEmpty(), TEXT("The Operation must contain a delegate with a parameter containing \"Response\" in the name."));	
			return ResponsePins;
		}

		TArray<UEdGraphPin*> GetErrorResponsePins(const UEdGraphNode* InNode)
		{
			static FString NegativeOutcomeNameStr = UE::WebAPI::Operation::NegativeOutcomeName.ToString();

			TArray<UEdGraphPin*> ErrorResponsePins = FindPins(InNode, NegativeOutcomeNameStr, EGPD_Output);
			ensureMsgf(!ErrorResponsePins.IsEmpty(), TEXT("The Operation must contain a delegate with a parameter containing \"%s\" in the name."), *NegativeOutcomeNameStr);	
			return ErrorResponsePins;
		}

		void CleanupPinNameInline(FString& InPinName)
		{
			constexpr static TCHAR DelimiterChar = TCHAR('_');

			int32 UnderscorePosition = -1;
			if(InPinName.FindLastChar(DelimiterChar, UnderscorePosition))
			{
				InPinName = InPinName.RightChop(UnderscorePosition + 1);
			}
		}
	}
}
