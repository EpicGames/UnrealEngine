// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "UObject/WeakFieldPtr.h"

template <typename T> class TSubclassOf;

class UWebAPIOperationObject;

namespace UE::WebAPI
{
	namespace Operation
	{
		/** Stores references to outcome delegate properties for a given operation. */
		struct FCachedOutcome
		{
		public:
			TWeakFieldPtr<const FMulticastDelegateProperty> PositiveOutcomeDelegate;
			TWeakFieldPtr<const FMulticastDelegateProperty> NegativeOutcomeDelegate;

			template <uint32 Index, typename TEnableIf<Index < 2>::Type* = nullptr>
			const TWeakFieldPtr<const FMulticastDelegateProperty>& Get() const
			{
				if constexpr (Index == 0)
				{
					return PositiveOutcomeDelegate;
				}
				else
				{
					return NegativeOutcomeDelegate;
				}
			}

			template <uint32 Index, typename TEnableIf<Index < 2>::Type* = nullptr>
			void Set(const FMulticastDelegateProperty* InDelegateProperty)
			{
				if constexpr (Index == 0)
				{
					PositiveOutcomeDelegate = InDelegateProperty;
				}
				else
				{
					NegativeOutcomeDelegate = InDelegateProperty;
				}
			}
		};

		/** String denoting a positive outcome, should match generated delegate name. */
		inline const static FName PositiveOutcomeName = TEXT("Success");

		/** String denoting a negative outcome, should match generated delegate name. */
		inline const static FName NegativeOutcomeName = TEXT("Error");

		/** Cached positive and negative (in order) delegates per operation class (name). */
		static TMap<FName, FCachedOutcome> CachedOutcomeDelegates;

		const FMulticastDelegateProperty* GetPositiveOutcomeDelegate(const TSubclassOf<UWebAPIOperationObject>& InOperationClass);

		template <class OperationType = UWebAPIOperationObject>
		const FMulticastDelegateProperty* GetPositiveOutcomeDelegate()
		{
			return GetPositiveOutcomeDelegate(OperationType::StaticClass());
		}
		
		const FMulticastDelegateProperty* GetNegativeOutcomeDelegate(const TSubclassOf<UWebAPIOperationObject>& InOperationClass);

		template <class OperationType = UWebAPIOperationObject>
		const FMulticastDelegateProperty* GetNegativeOutcomeDelegate()
		{
			return GetNegativeOutcomeDelegate(OperationType::StaticClass());
		}

		UFunction* GetOutcomeDelegateSignatureFunction(const TSubclassOf<UWebAPIOperationObject>& InOperationClass);
	} 

	namespace Graph
	{
		/** Connects all pins linked to src, to dst. */
		void TransferPinConnections(const UEdGraphPin* InSrc, UEdGraphPin* InDst);

		/** Connects all pins linked to all src pins, to matching (same named) dst pins. */
		void TransferPins(const TArray<UEdGraphPin*>& InSrcPins, const TArray<UEdGraphPin*>& InDstPins);

		/** Split multiple pins according to various criteria, like unwrapping single property structs, etc. */
		bool SplitPins(const TArray<UEdGraphPin*>& InPins);

		/** Split single pin according to various criteria, like unwrapping single property structs, etc. */
		bool SplitPin(UEdGraphPin* InPin);

		/** Filter the inputs pins by the input execution chain (like "Hide Unrelated"). */
		TArray<UEdGraphPin*> FilterPinsByRelated(const UEdGraphPin* InExecutionPin, const TArray<UEdGraphPin*>& InPinsToFilter);

		/** Find a single pin according to various criteria. */
		UEdGraphPin* FindPin(
			const UEdGraphNode* InNode,
			const FName& InName,
			const EEdGraphPinDirection& InDirection = EEdGraphPinDirection::EGPD_MAX,
			const FName& InCategory = NAME_All,
			bool bFindPartial = false);

		/** Find multiple pins according to various criteria. */
		TArray<UEdGraphPin*> FindPins(
			const UEdGraphNode* InNode,
			const FString& InName,
			const EEdGraphPinDirection& InPinDirection,
			bool bOnlySplitPins = true);

		/** Get all (non-exec, non-error) output pins, will return empty if this uses Callbacks. */
		TArray<UEdGraphPin*> GetResponsePins(const UEdGraphNode* InNode);

		/** Get all error output pins, will return empty if this uses Callbacks. */
		TArray<UEdGraphPin*> GetErrorResponsePins(const UEdGraphNode* InNode);

		/** Returns the last part of a string, split by underscores. */
		void CleanupPinNameInline(FString& InPinName);
	}
}
