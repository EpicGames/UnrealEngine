// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebAPIBlueprintGraph : ModuleRules
{
	public WebAPIBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Json",
				"Slate",
				"SlateCore",
				"WebAPI",
			}
		);
		
		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetTools",
					"BlueprintGraph",
					"EditorStyle",
					"GraphEditor",
					"Kismet",
					"KismetCompiler",
					"KismetWidgets",
					"Projects",
					"PropertyEditor",
					"ToolMenus",
					"UnrealEd"
				});
		}
	}
}
