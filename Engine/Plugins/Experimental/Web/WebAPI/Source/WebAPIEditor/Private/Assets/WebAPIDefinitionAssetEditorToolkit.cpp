// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDefinitionAssetEditorToolkit.h"

#include "AssetEditorModeManager.h"
#include "CodeGen/WebAPICodeGenerator.h"
#include "PropertyEditorModule.h"
#include "DetailsViewArgs.h"
#include "WebAPIEditorCommands.h"
#include "Dom/WebAPISchema.h"
#include "WebAPIEditorModule.h"
#include "Async/Async.h"
#include "Details/ViewModels/WebAPICodeViewModel.h"
#include "Details/Widgets/SWebAPICodeView.h"
#include "Details/Widgets/SWebAPIMessageLog.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "Widgets/Docking/SDockTab.h"
#include "WebAPIDefinition.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "WebAPIDefinitionAssetEditorToolkit"

// Main details tab
const FName FWebAPIDefinitionAssetEditorToolkit::DetailsTabID(TEXT("WebAPIDefinitionToolkit_Details"));

// MessageLog tab
const FName FWebAPIDefinitionAssetEditorToolkit::LogTabID(TEXT("WebAPIDefinitionToolkit_Log"));

// Code view tab
const FName FWebAPIDefinitionAssetEditorToolkit::CodeTabID(TEXT("WebAPIDefinitionToolkit_Code"));

FWebAPIDefinitionAssetEditorToolkit::FWebAPIDefinitionAssetEditorToolkit()
	: Definition(nullptr)
{
	LayoutExtender = MakeShared<FLayoutExtender>();
}

FWebAPIDefinitionAssetEditorToolkit::~FWebAPIDefinitionAssetEditorToolkit()
{
	GEditor->UnregisterForUndo(this);
}

void FWebAPIDefinitionAssetEditorToolkit::Initialize(const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost, UWebAPIDefinition* InDefinition)
{
	check(InDefinition);
	Definition = TStrongObjectPtr(InDefinition);

	// create tab layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_WebAPIDefinitionEditor_v1")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.8f)
			->AddTab(DetailsTabID, ETabState::ClosedTab)
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.2f)
			->AddTab(CodeTabID, ETabState::ClosedTab)
			->AddTab(LogTabID, ETabState::ClosedTab)
		)
	);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;

	FAssetEditorToolkit::InitAssetEditor(
			InMode,
			InToolkitHost,
			FWebAPIEditorModule::AppIdentifier,
			StandaloneDefaultLayout,
			bCreateDefaultStandaloneMenu,
			bCreateDefaultToolbar,
			InDefinition
		);

	FWebAPIEditorModule& WebAPIEditorModule = FModuleManager::LoadModuleChecked<FWebAPIEditorModule>("WebAPIEditor");
	AddMenuExtender(WebAPIEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	const TSharedPtr<FExtender> ToolbarExtender = WebAPIEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	ExtendToolbar(ToolbarExtender);
	AddToolbarExtender(ToolbarExtender);
	
	RegenerateMenusAndToolbars();

	WebAPIEditorModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender);

	// Support undo/redo
	GEditor->RegisterForUndo(this);

	SetupCommands();
}

FName FWebAPIDefinitionAssetEditorToolkit::GetToolkitFName() const
{
	return FName(TEXT("WebAPI Definition"));
}

FText FWebAPIDefinitionAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "WebAPIDefinitionEditor");
}

FString FWebAPIDefinitionAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "WebAPIDefinitionEditor ").ToString();
}

FLinearColor FWebAPIDefinitionAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FWebAPIDefinitionAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	if(!AssetEditorTabsCategory.IsValid())
	{
		// Use the first child category of the local workspace root if there is one, otherwise use the root itself
		const TArray<TSharedRef<FWorkspaceItem>>& LocalCategories = InTabManager->GetLocalWorkspaceMenuRoot()->GetChildItems();
		AssetEditorTabsCategory = LocalCategories.Num() > 0 ? LocalCategories[0] : InTabManager->GetLocalWorkspaceMenuRoot();
	}

	// Details
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	//DetailsView = StaticCastSharedPtr<FWebAPIDefinitionDetailsCustomization>(RawDetailsView);
	

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	// Log
	{
		LogView = SNew(SWebAPIMessageLog, Definition->GetMessageLog().ToSharedRef());

		InTabManager->RegisterTabSpawner(LogTabID, FOnSpawnTab::CreateSP(this, &FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Log))
		.SetDisplayName(LOCTEXT("Log", "Log"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef());
	}

	// Code
	{
		CodeViewModel = FWebAPICodeViewModel::Create(Definition.Get());
		CodeView = SNew(SWebAPICodeView, CodeViewModel.ToSharedRef());
		
		InTabManager->RegisterTabSpawner(CodeTabID, FOnSpawnTab::CreateSP(this, &FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Code))
		.SetDisplayName(LOCTEXT("Code", "Code"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef());
	}
}

void FWebAPIDefinitionAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(DetailsTabID);
	DetailsView.Reset();
	
	InTabManager->UnregisterTabSpawner(LogTabID);
	
	InTabManager->UnregisterTabSpawner(CodeTabID);
}

void FWebAPIDefinitionAssetEditorToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FAssetEditorModeManager>();
}

void FWebAPIDefinitionAssetEditorToolkit::RegisterToolbar()
{
}

void FWebAPIDefinitionAssetEditorToolkit::SetupCommands()
{
	GetToolkitCommands()->MapAction(
	FWebAPIEditorCommands::Get().Generate,
	FExecuteAction::CreateRaw(this, &FWebAPIDefinitionAssetEditorToolkit::Generate));
}

void FWebAPIDefinitionAssetEditorToolkit::ExtendToolbar(const TSharedPtr<FExtender> InExtender)
{
	struct FToolbarHelper
	{
		static void PopulateToolbar(FToolBarBuilder& InToolbarBuilder, FWebAPIDefinitionAssetEditorToolkit* Toolkit)
		{
			InToolbarBuilder.BeginSection("Toolbar");
			{
				InToolbarBuilder.AddToolBarButton(
					FWebAPIEditorCommands::Get().Generate,
					NAME_None,
					TAttribute<FText>(),
					TAttribute<FText>(Toolkit, &FWebAPIDefinitionAssetEditorToolkit::GetGenerateStatusTooltip),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Blueprint.CompileStatus.Background"));
			}
			InToolbarBuilder.EndSection();
		}
	};
	
	InExtender->AddToolBarExtension(
	"Asset",
	EExtensionHook::Before,
	GetToolkitCommands(),
	FToolBarExtensionDelegate::CreateStatic(&FToolbarHelper::PopulateToolbar, this));
}

void FWebAPIDefinitionAssetEditorToolkit::Generate() const
{
	// Clear log before Generate
	Definition->GetMessageLog()->ClearLog();
	
	const TScriptInterface<IWebAPICodeGeneratorInterface>& CodeGenerator = Definition->GetGeneratorSettings().GetGeneratorClass();
	if(!CodeGenerator)
	{
		Definition->GetWebAPISchema()->GetMessageLog()->LogWarning(LOCTEXT("NoCodeGeneratorsAvailable", "No code generators are available."), LogName);
		return;
	}

	// Don't generate if the schema is invalid
	FDataValidationContext Context;
	if(Definition->GetWebAPISchema()->IsDataValid(Context) == EDataValidationResult::Invalid)
	{
		Definition->GetWebAPISchema()->GetMessageLog()->LogWarning(LOCTEXT("SchemaInvalid", "There are one or more validation errors in the asset, cannot Generate code."), LogName);
		return;
	}

	CodeGenerator->IsAvailable().Next([CodeGenerator = CodeGenerator, Definition = Definition](bool bIsAvailable)
	{
		if(!bIsAvailable)
		{
			FString CodeGeneratorName = CodeGenerator.GetObject()->GetName();
			CodeGeneratorName.RemoveFromStart(TEXT("Default__"));
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("Name"), FText::FromString(CodeGeneratorName));
			Definition->GetWebAPISchema()->GetMessageLog()->LogError(FText::Format(LOCTEXT("CodeGeneratorNotAvailable", "The selected code generator \"{Name}\" is not available. Aborting."), Args), LogName);
			return;
		}
		
		CodeGenerator->Generate(Definition.Get())
		.Next([Definition = Definition.Get()](auto)
		{
			AsyncTask(ENamedThreads::GameThread, [Definition]
			{
				Definition->GetWebAPISchema()->GetMessageLog()->LogInfo(LOCTEXT("CodeGenerationComplete", "Code generation complete!"), LogName);
			});
		});
	});
}

FSlateIcon FWebAPIDefinitionAssetEditorToolkit::GetGenerateStatusImage() const
{
	return {};
}

FText FWebAPIDefinitionAssetEditorToolkit::GetGenerateStatusTooltip() const
{
	return FText::GetEmpty();
}

void FWebAPIDefinitionAssetEditorToolkit::SetEditingObject(UObject* InObject)
{
}

TSharedRef<SDockTab> FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Details(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == DetailsTabID);

	DetailsView->SetObject(Definition.Get());

	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Log(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == LogTabID);

	return SNew(SDockTab)
		.Label(LOCTEXT("LogTitle", "Log"))
		[
			LogView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FWebAPIDefinitionAssetEditorToolkit::SpawnTab_Code(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == CodeTabID);

	return SNew(SDockTab)
		.Label(LOCTEXT("CodeViewTitle", "Code"))
		[
			CodeView.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
