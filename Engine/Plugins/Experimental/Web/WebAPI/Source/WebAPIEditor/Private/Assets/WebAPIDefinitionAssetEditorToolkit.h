// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/StrongObjectPtr.h"

class FSpawnTabArgs;
class UWebAPIDefinition;

class FWebAPIDefinitionAssetEditorToolkit
	: public FAssetEditorToolkit
	, public FEditorUndoClient
{
public:
	FWebAPIDefinitionAssetEditorToolkit();
	virtual ~FWebAPIDefinitionAssetEditorToolkit() override;

	/** Initializes the toolkit for the given WebAPI Definition. */
	void Initialize(const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost, UWebAPIDefinition* InDefinition);

	//~ Begin IToolkit Interface.
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;	
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface.

	//~ Begin IToolkit Interface.
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	//~ Begin IToolkit Interface.

protected:
	virtual void SetEditingObject(class UObject* InObject);
	virtual void RegisterToolbar();

	//~ Begin IToolkit Interface.
	virtual void CreateEditorModeManager() override;
	//~ End IToolkit Interface.

/** Commands. */
public:
	/** Builds the toolbar widget. */
	void SetupCommands();
	void ExtendToolbar(const TSharedPtr<FExtender> InExtender);

	/** Execute the Generate command. */
	void Generate() const;
	
	/** The icon representing the current status of the Generate command. */
	FSlateIcon GetGenerateStatusImage() const;

	/** The tooltip for the current status of the Generate command. */
	FText GetGenerateStatusTooltip() const;

/** Tabs. */
public:
	/** Main details tab. */
	static const FName DetailsTabID;

	/** MessageLog tab. */
	static const FName LogTabID;

	/** Code view tab. */
	static const FName CodeTabID;

protected:
	/** Main details tab. */
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args) const;

	/** MessageLog tab. */
	TSharedRef<SDockTab> SpawnTab_Log(const FSpawnTabArgs& Args) const;

	/** Code view tab. */
	TSharedRef<SDockTab> SpawnTab_Code(const FSpawnTabArgs& Args) const;

protected:
	static constexpr const TCHAR* LogName = TEXT("Asset Editor");
	
	/** Property View. */
	TSharedPtr<class IDetailsView> DetailsView;

	/** Log View. */
	TSharedPtr<class SWebAPIMessageLog> LogView;

	/** Code ViewModel. */
	TSharedPtr<class FWebAPICodeViewModel> CodeViewModel;
	
	/** Code View. */
	TSharedPtr<class SWebAPICodeView> CodeView;

	/** Extender for adding to the default layout for this asset editor */
	TSharedPtr<FLayoutExtender> LayoutExtender;

	/** The WebAPI Definition being edited. */
	TStrongObjectPtr<UWebAPIDefinition> Definition;
};
