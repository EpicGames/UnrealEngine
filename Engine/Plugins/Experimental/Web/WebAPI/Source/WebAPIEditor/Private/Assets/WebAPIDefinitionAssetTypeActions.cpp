// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDefinitionAssetTypeActions.h"

#include "IWebAPIEditorModule.h"
#include "WebAPIDefinition.h"
#include "WebAPIDefinitionAssetEditorToolkit.h"

#define LOCTEXT_NAMESPACE "WebAPIDefinitionAssetTypeActions"

FText FWebAPIDefinitionAssetTypeActions::GetName() const
{
	return LOCTEXT("TypeActionsName", "WebAPIDefinition");
}

FColor FWebAPIDefinitionAssetTypeActions::GetTypeColor() const
{
	return FColor(38, 77, 228);
}

UClass* FWebAPIDefinitionAssetTypeActions::GetSupportedClass() const
{
	return UWebAPIDefinition::StaticClass();
}

uint32 FWebAPIDefinitionAssetTypeActions::GetCategories()
{
	return IWebAPIEditorModuleInterface::Get().GetAssetCategory();
}

void FWebAPIDefinitionAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> InEditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = InEditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UWebAPIDefinition* Definition = Cast<UWebAPIDefinition>(*ObjIt))
		{
			const TSharedPtr<FWebAPIDefinitionAssetEditorToolkit> NewEditor = MakeShared<FWebAPIDefinitionAssetEditorToolkit>();
			NewEditor->Initialize(Mode, InEditWithinLevelEditor, Definition);
		}
	}
}

#undef LOCTEXT_NAMESPACE
