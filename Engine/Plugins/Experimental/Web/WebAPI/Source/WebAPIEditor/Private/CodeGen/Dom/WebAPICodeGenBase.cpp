// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "UObject/NameTypes.h"

FWebAPICodeGenBase::FWebAPICodeGenBase()
{
	Specifiers.Add(TEXT("BlueprintType"), TEXT(""));
	Metadata.Add(TEXT("DisplayName"), GetName(true));
}

void FWebAPICodeGenBase::SetModule(const FString& InModule)
{
	Module = InModule;
}

// Wraps the Array overload
void FWebAPICodeGenBase::GetIncludePaths(TSet<FString>& OutIncludePaths) const
{
	TArray<FString> IncludePathsArray;
	GetIncludePaths(IncludePathsArray);
	OutIncludePaths.Append(IncludePathsArray);
}

const FName& FWebAPICodeGenBase::GetTypeName()
{
	static const FName None = NAME_None;
	return None;
}
