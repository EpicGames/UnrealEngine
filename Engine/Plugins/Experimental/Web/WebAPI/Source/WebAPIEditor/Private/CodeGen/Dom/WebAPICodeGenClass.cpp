// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenClass.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "CodeGen/Dom/WebAPICodeGenFunction.h"
#include "CodeGen/Dom/WebAPICodeGenStruct.h"

FWebAPICodeGenClass::FWebAPICodeGenClass(const FWebAPICodeGenStruct& InStruct)
	: Super::FWebAPICodeGenStruct(InStruct)
{
}

const TSharedPtr<FWebAPICodeGenFunction>& FWebAPICodeGenClass::FindOrAddFunction(const FWebAPINameVariant& InName)
{
	const TSharedPtr<FWebAPICodeGenFunction>* FoundFunction = Functions.FindByPredicate([&InName](const TSharedPtr<FWebAPICodeGenFunction>& InFunction)
	{
		return InFunction->Name == InName;
	});

	if(!FoundFunction)
	{
		const TSharedPtr<FWebAPICodeGenFunction>& Function = Functions.Emplace_GetRef(MakeShared<FWebAPICodeGenFunction>());
		Function->Name = InName;
		return Function;
	}

	return *FoundFunction;
}
