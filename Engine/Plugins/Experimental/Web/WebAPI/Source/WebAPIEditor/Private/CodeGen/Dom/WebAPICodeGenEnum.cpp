// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenEnum.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "WebAPIDefinition.h"
#include "WebAPIEditorUtilities.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIType.h"

void FWebAPICodeGenEnumValue::FromWebAPI(const UWebAPIEnumValue* InSrcEnumValue)
{
	check(InSrcEnumValue);

	Name = InSrcEnumValue->Name.ToMemberName(TEXT("EV"));
	DisplayName = InSrcEnumValue->Name.GetDisplayName();
	JsonName = InSrcEnumValue->Name.GetJsonName();
	Description = InSrcEnumValue->Description;
}

void FWebAPICodeGenEnum::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);
	OutModules.Append(Name.TypeInfo->Modules);
}

void FWebAPICodeGenEnum::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);
}

FString FWebAPICodeGenEnum::GetName(bool bJustName)
{
	return Name.ToString(true) + (Name.HasTypeInfo() ? Name.TypeInfo->Suffix : TEXT(""));
}

void FWebAPICodeGenEnum::FromWebAPI(const UWebAPIEnum* InSrcEnum)
{
	check(InSrcEnum);
	
	Name = InSrcEnum->Name;	
	Description = InSrcEnum->Description;

	Metadata.FindOrAdd(TEXT("DisplayName"), Name.GetDisplayName());
	if(Name.HasTypeInfo() && !Name.TypeInfo->Namespace.IsEmpty())
	{
		Namespace = Name.TypeInfo->Namespace;
		Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
	}

	DefaultValue = InSrcEnum->GetDefaultValue(true);

	if(InSrcEnum->bIsRequired)
	{
		Metadata.FindOrAdd(TEXT("DefaultValue"), DefaultValue);
	}
	
	for(const TObjectPtr<UWebAPIEnumValue>& SrcEnumValue : InSrcEnum->Values)
	{
		FWebAPICodeGenEnumValue DstEnumValue;
		DstEnumValue.FromWebAPI(SrcEnumValue);
		Values.Emplace(MoveTemp(DstEnumValue));
	}

	{
		UWebAPIDefinition* OwningDefinition = InSrcEnum->GetTypedOuter<UWebAPIDefinition>();
		check(OwningDefinition);

		const FString UnsetEnumValueName = OwningDefinition->GetProviderSettings().GetUnsetEnumValueName();
		
		FWebAPICodeGenEnumValue DstEnumValue;
		DstEnumValue.Name = UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(TEXT("EV")) + TEXT("_") + UnsetEnumValueName;
		DstEnumValue.DisplayName = TEXT("(") + UnsetEnumValueName + TEXT(")");
		DstEnumValue.JsonName = TEXT("none");
		DstEnumValue.Description = TEXT("Represents an unset value for an optional enum.");
		DstEnumValue.IntegralValue = 255;
		
		Values.Emplace(MoveTemp(DstEnumValue));
	}
}
