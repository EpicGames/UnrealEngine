// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenFile.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "Misc/Paths.h"

void FWebAPICodeGenFile::AddItem(const TSharedPtr<FWebAPICodeGenBase>& InItem)
{
	InItem->GetModuleDependencies(Modules);
	InItem->GetIncludePaths(IncludePaths);
	SubItems.Add(InItem);
}

FString FWebAPICodeGenFile::GetFullPath()
{
	return FPaths::Combine(BaseFilePath, RelativeFilePath, FileName + TEXT(".") + FileType);
}

void FWebAPICodeGenFile::SetModule(const FString& InModule)
{
	Super::SetModule(InModule);

	for(const TSharedPtr<FWebAPICodeGenBase>& SubItem : SubItems)
	{
		SubItem->SetModule(InModule);
	}
}

void FWebAPICodeGenFile::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);

	for(const TSharedPtr<FWebAPICodeGenBase>& SubItem : SubItems)
	{
		SubItem->GetIncludePaths(OutIncludePaths);
	}

	OutIncludePaths.Sort(); // alpha-numeric sort
}

void FWebAPICodeGenFile::GetIncludePaths(TSet<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);
}
