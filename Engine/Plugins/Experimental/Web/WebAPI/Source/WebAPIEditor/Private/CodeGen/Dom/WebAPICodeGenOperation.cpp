// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenOperation.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "IWebAPIEditorModule.h"
#include "CodeGen/Dom/WebAPICodeGenStruct.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPIType.h"
#include "Dom/WebAPITypeRegistry.h"

void FWebAPICodeGenOperationParameter::FromWebAPI(const UWebAPIOperationParameter* InSrcOperationParameter)
{
	Super::FromWebAPI(InSrcOperationParameter);
	
	check(InSrcOperationParameter);
	
	Name = InSrcOperationParameter->Name;
	Description = InSrcOperationParameter->Description;
	Storage = InSrcOperationParameter->Storage;
	MediaType = InSrcOperationParameter->MediaType;

	Metadata.FindOrAdd(TEXT("DisplayName")) = Name.GetDisplayName();
	Metadata.FindOrAdd(TEXT("JsonName")) = Name.GetJsonName();
}

const TSharedPtr<FWebAPICodeGenOperationParameter>& FWebAPICodeGenOperationRequest::FindOrAddParameter(const FString& InName)
{
	TSharedPtr<FWebAPICodeGenOperationParameter>* FoundParameter = Parameters.FindByPredicate([&InName](const TSharedPtr<FWebAPICodeGenOperationParameter>& InParameter)
	{
		return InParameter->Name.NameInfo.ToMemberName().Equals(InName, ESearchCase::IgnoreCase);
	});

	if(!FoundParameter)
	{
		const TSharedPtr<FWebAPICodeGenOperationParameter>& Parameter = Parameters.Emplace_GetRef(MakeShared<FWebAPICodeGenOperationParameter>());
		Parameter->Name = InName;
		Parameter->Name.NameInfo.Name = InName;

		// Also add as property for generation purposes
		Properties.Add(Parameter);

		return Parameter;
	}

	return *FoundParameter;
}

void FWebAPICodeGenOperationRequest::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);

	for(const TSharedPtr<FWebAPICodeGenOperationParameter>& Parameter : Parameters)
	{
		Parameter->GetModuleDependencies(OutModules);		
	}
}

void FWebAPICodeGenOperationRequest::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);

	for(const TSharedPtr<FWebAPICodeGenOperationParameter>& Parameter : Parameters)
	{
		Parameter->GetIncludePaths(OutIncludePaths);		
	}
}

void FWebAPICodeGenOperationRequest::SetModule(const FString& InModule)
{
	Super::SetModule(InModule);

	for(const TSharedPtr<FWebAPICodeGenOperationParameter>& Parameter : Parameters)
	{
		Parameter->SetModule(InModule);		
	}
}

void FWebAPICodeGenOperationRequest::FromWebAPI(const UWebAPIOperationRequest* InSrcOperationRequest)
{
	Super::FromWebAPI(InSrcOperationRequest);
	Metadata.Remove(TEXT("JsonName"));
	
	check(InSrcOperationRequest);
	
	Name = InSrcOperationRequest->Name;
	Description = InSrcOperationRequest->Description;

	for(const TObjectPtr<UWebAPIOperationParameter>& SrcParameter : InSrcOperationRequest->Parameters)
	{
		const TSharedPtr<FWebAPICodeGenOperationParameter>& DstParameter = FindOrAddParameter(SrcParameter->Name.NameInfo.ToMemberName());
		DstParameter->FromWebAPI(SrcParameter);

		if(!Namespace.IsEmpty())
		{
			DstParameter->Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
		}
	}
}

void FWebAPICodeGenOperationResponse::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);
}

void FWebAPICodeGenOperationResponse::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);
}

void FWebAPICodeGenOperationResponse::SetModule(const FString& InModule)
{
	Super::SetModule(InModule);
}

void FWebAPICodeGenOperationResponse::FromWebAPI(const UWebAPIOperationResponse* InSrcOperationResponse)
{
	Super::FromWebAPI(InSrcOperationResponse);
	Metadata.Remove(TEXT("JsonName"));
	
	check(InSrcOperationResponse);

	Name = InSrcOperationResponse->Name;
	ResponseCode = InSrcOperationResponse->Code;
	Description = InSrcOperationResponse->Description;
	Message = Description;
	Base = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->WebAPIMessageResponse;

	FString ResponseCodeStr = InSrcOperationResponse->Code > 0 ? FString::FormatAsNumber(InSrcOperationResponse->Code) : TEXT("Default");

	Metadata.FindOrAdd(TEXT("DisplayName")) = FString::Printf(TEXT("Response (%s)"), *ResponseCodeStr);
	Metadata.FindOrAdd(TEXT("ResponseCode")) = ResponseCodeStr;

	if(!Namespace.IsEmpty())
	{
		Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
	}
}

const TSharedPtr<FWebAPICodeGenOperationResponse>& FWebAPICodeGenOperation::FindOrAddResponse(const uint32& InResponseCode)
{
	const TSharedPtr<FWebAPICodeGenOperationResponse>* FoundResponse = Responses.FindByPredicate([&InResponseCode](const TSharedPtr<FWebAPICodeGenOperationResponse>& InResponse)
	{
		return InResponse->ResponseCode == InResponseCode;
	});

	if(!FoundResponse)
	{
		const TSharedPtr<FWebAPICodeGenOperationResponse>& Response = Responses.Emplace_GetRef(MakeShared<FWebAPICodeGenOperationResponse>());
		Response->ResponseCode = InResponseCode;
		return Response;
	}

	return *FoundResponse;
}

void FWebAPICodeGenOperation::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);

	for(const TSharedPtr<FWebAPICodeGenOperationRequest>& Request : Requests)
	{
		Request->GetModuleDependencies(OutModules);		
	}

	for(const TSharedPtr<FWebAPICodeGenOperationResponse>& Response : Responses)
	{
		Response->GetModuleDependencies(OutModules);
	}
}

void FWebAPICodeGenOperation::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);

	OutIncludePaths.Add(TEXT("Engine/Engine.h"));

	for(const TSharedPtr<FWebAPICodeGenOperationRequest>& Request : Requests)
	{
		Request->GetIncludePaths(OutIncludePaths);		
	}

	for(const TSharedPtr<FWebAPICodeGenOperationResponse>& Response : Responses)
	{
		Response->GetIncludePaths(OutIncludePaths);		
	}
}

void FWebAPICodeGenOperation::SetModule(const FString& InModule)
{
	Super::SetModule(InModule);

	for(const TSharedPtr<FWebAPICodeGenOperationRequest>& Request : Requests)
	{
		Request->SetModule(InModule);		
	}

	for(const TSharedPtr<FWebAPICodeGenOperationResponse>& Response : Responses)
	{
		Response->SetModule(InModule);		
	}
}

FString FWebAPICodeGenOperation::GetName(bool bJustName)
{
	return Name.ToString(true) + (Name.HasTypeInfo() ? Name.TypeInfo->Suffix : TEXT(""));
}

void FWebAPICodeGenOperation::FromWebAPI(const UWebAPIOperation* InSrcOperation)
{
	check(InSrcOperation);

	Name = InSrcOperation->Name;
	Description = InSrcOperation->Description;
	if(Name.HasTypeInfo() && !Name.TypeInfo->Namespace.IsEmpty())
	{
		Namespace = Name.TypeInfo->Namespace;
	}

	Verb = InSrcOperation->Verb;
	Path = InSrcOperation->Path;

	Metadata.FindOrAdd(TEXT("DisplayName")) = Name.GetDisplayName();
	Metadata.FindOrAdd(TEXT("Service")) = InSrcOperation->Service->Name.GetDisplayName();

	const TSharedPtr<FWebAPICodeGenOperationRequest>& Request = Requests.Emplace_GetRef(MakeShared<FWebAPICodeGenOperationRequest>());
	Request->FromWebAPI(InSrcOperation->Request);

	if(!Namespace.IsEmpty())
	{
		Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
	}
	
	for(const TObjectPtr<UWebAPIOperationResponse>& SrcResponse : InSrcOperation->Responses)
	{
		const TSharedPtr<FWebAPICodeGenOperationResponse>& DstResponse = FindOrAddResponse(SrcResponse->Code);
		DstResponse->FromWebAPI(SrcResponse);

		DstResponse->Metadata.FindOrAdd(TEXT("Service")) = InSrcOperation->Service->Name.GetDisplayName();
	}
}
