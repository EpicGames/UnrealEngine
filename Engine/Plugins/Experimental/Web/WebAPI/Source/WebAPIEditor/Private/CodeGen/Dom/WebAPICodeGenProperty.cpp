// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenProperty.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "Dom/WebAPIModel.h"
#include "Dom/WebAPIType.h"

FWebAPICodeGenProperty::FWebAPICodeGenProperty()
{
	Specifiers.Remove(TEXT("BlueprintType"));
	Specifiers.Add(TEXT("EditAnywhere"));
	Specifiers.Add(TEXT("BlueprintReadWrite"));
}

void FWebAPICodeGenProperty::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);
	if(Type.HasTypeInfo())
	{
		OutModules.Append(Type.TypeInfo->Modules);
	}
}

void FWebAPICodeGenProperty::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);
	if(Type.HasTypeInfo())
	{
		OutIncludePaths.Append(Type.TypeInfo->IncludePaths.Array());
	}
}

void FWebAPICodeGenProperty::FromWebAPI(const UWebAPIProperty* InSrcProperty)
{
	check(InSrcProperty);
	
	Name = InSrcProperty->Name.ToMemberName();
	Description = InSrcProperty->Description;
	Type = InSrcProperty->Type;
	DefaultValue = InSrcProperty->GetDefaultValue(true);
	bIsArray = InSrcProperty->bIsArray;
	bIsMixin = InSrcProperty->bIsMixin;

	if(bIsMixin)
	{
		Metadata.Add(TEXT("Mixin"));
	}

	if(InSrcProperty->bIsReadOnly)
	{
		Specifiers.Remove(TEXT("EditAnywhere"));
		Specifiers.Remove(TEXT("BlueprintReadWrite"));

		Specifiers.Add(TEXT("VisibleAnywhere"));
		Specifiers.Add(TEXT("BlueprintReadOnly"));
	}

	if(InSrcProperty->bIsRequired)
	{
		bIsRequired = true;
		Metadata.FindOrAdd(TEXT("DefaultValue"),InSrcProperty->GetDefaultValue(true).ReplaceQuotesWithEscapedQuotes());
	}

	Metadata.FindOrAdd(TEXT("DisplayName")) = Name.GetDisplayName();
	Metadata.FindOrAdd(TEXT("JsonName")) = Name.GetJsonName();

	if(!InSrcProperty->bIsRequired)
	{
		Metadata.FindOrAdd(TEXT("Optional"));		
	}
}
