// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenSettings.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "UObject/Package.h"
#include "WebAPIDefinition.h"
#include "WebAPIEditorUtilities.h"
#include "Dom/WebAPIType.h"
#include "Templates/SubclassOf.h"

void FWebAPICodeGenSettings::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);

	OutModules.Add(TEXT("DeveloperSettings"));

	TArray<FString> BaseClassModuleDependencies;
	if(ParentType.IsValid() && UE::WebAPI::FWebAPIEditorUtilities::Get()->GetModulesForClass(TSubclassOf<UObject>(ParentType.Get()), BaseClassModuleDependencies))
	{
		OutModules.Append(BaseClassModuleDependencies);
	}
}

void FWebAPICodeGenSettings::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);

	OutIncludePaths.Add(TEXT("Engine/DeveloperSettings.h"));

	TArray<FString> BaseClassHeaders;
	if(ParentType.IsValid() && UE::WebAPI::FWebAPIEditorUtilities::Get()->GetHeadersForClass(TSubclassOf<UObject>(ParentType.Get()), BaseClassHeaders))
	{
		OutIncludePaths.Append(BaseClassHeaders);
	}
}

void FWebAPICodeGenSettings::FromWebAPI(const UWebAPIDefinition* InSrcModel)
{
	check(InSrcModel);

	Name.TypeInfo = NewObject<UWebAPITypeInfo>(GetTransientPackage());
	Name.TypeInfo->SetName(FString::Printf(TEXT("%sSettings"), *InSrcModel->GetGeneratorSettings().TargetModule.Name));
	Name.TypeInfo->Prefix = TEXT("U");
	Name.TypeInfo->Namespace = InSrcModel->GetGeneratorSettings().GetNamespace();

	if(InSrcModel->GetGeneratorSettings().GeneratedSettingsBaseClass.IsValid())
	{
		ParentType = InSrcModel->GetGeneratorSettings().GeneratedSettingsBaseClass;	
	}
	
	Specifiers.FindOrAdd(TEXT("BlueprintType"));
	Specifiers.FindOrAdd(TEXT("Config")) = TEXT("Engine");
	Specifiers.FindOrAdd(TEXT("DefaultConfig"));
	
	Metadata.FindOrAdd(TEXT("DisplayName")) = Name.GetDisplayName();

	const UWebAPISchema* Schema = InSrcModel->GetWebAPISchema();
	Host = Schema->Host;
	BaseUrl = Schema->BaseUrl;
	UserAgent = Schema->UserAgent;
	DateTimeFormat = Schema->DateTimeFormat;
	Schemes = Schema->URISchemes;
}
