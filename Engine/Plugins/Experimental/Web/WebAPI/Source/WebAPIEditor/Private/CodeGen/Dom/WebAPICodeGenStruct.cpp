// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/Dom/WebAPICodeGenStruct.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "Dom/WebAPIModel.h"
#include "CodeGen/Dom/WebAPICodeGenProperty.h"
#include "Dom/WebAPIType.h"

const TSharedPtr<FWebAPICodeGenProperty>& FWebAPICodeGenStruct::FindOrAddProperty(const FWebAPINameVariant& InName)
{
	const TSharedPtr<FWebAPICodeGenProperty>* FoundProperty = Properties.FindByPredicate([&InName](const TSharedPtr<FWebAPICodeGenProperty>& InFunction)
	{
		return InFunction->Name == InName;
	});

	if(!FoundProperty)
	{
		const TSharedPtr<FWebAPICodeGenProperty>& Property = Properties.Emplace_GetRef(MakeShared<FWebAPICodeGenProperty>());
		Property->Name = InName;
		return Property;
	}

	return *FoundProperty;
}

void FWebAPICodeGenStruct::GetModuleDependencies(TSet<FString>& OutModules) const
{
	Super::GetModuleDependencies(OutModules);

	check(Name.HasTypeInfo()); // Required
	
	OutModules.Append(Name.TypeInfo->Modules);
	for(const TSharedPtr<FWebAPICodeGenProperty>& Property : Properties)
	{
		Property->GetModuleDependencies(OutModules);
	}
}

void FWebAPICodeGenStruct::GetIncludePaths(TArray<FString>& OutIncludePaths) const
{
	Super::GetIncludePaths(OutIncludePaths);
	OutIncludePaths.AddUnique(TEXT("CoreMinimal.h"));
	for(const TSharedPtr<FWebAPICodeGenProperty>& Property : Properties)
	{
		Property->GetIncludePaths(OutIncludePaths);
	}
	
	if(Base.HasTypeInfo())
	{
		OutIncludePaths.Append(Base.TypeInfo->IncludePaths.Array());		
	}
}

FString FWebAPICodeGenStruct::GetName(bool bJustName)
{
	return Name.ToString(true) + (Name.HasTypeInfo() ? Name.TypeInfo->Suffix : TEXT(""));
}

bool FWebAPICodeGenStruct::FindRecursiveProperties(TArray<TSharedPtr<FWebAPICodeGenProperty>>& OutProperties)
{
	bool bHasRecursiveType = false;
	for(const TSharedPtr<FWebAPICodeGenProperty>& Property : Properties)
	{
		if(Property->Type == this->Name)
		{
			Property->bIsRecursiveType = true;
			OutProperties.Add(Property);
			bHasRecursiveType = true;
		}
	}
	
	return bHasRecursiveType;
}

void FWebAPICodeGenStruct::FromWebAPI(const UWebAPIModel* InSrcModel)
{
	check(InSrcModel);
	check(InSrcModel->Name.HasTypeInfo()); // Required
	
	Name = InSrcModel->Name;
	Description = InSrcModel->Description;
	if(Name.HasTypeInfo() && !Name.TypeInfo->Namespace.IsEmpty())
	{
		Namespace =  Name.TypeInfo->Namespace;
	}

	Metadata.FindOrAdd(TEXT("DisplayName")) = Name.GetDisplayName();
	Metadata.FindOrAdd(TEXT("JsonName")) = Name.GetJsonName();

	if(!Namespace.IsEmpty())
	{
		Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
	}

	for(const TObjectPtr<UWebAPIProperty>& SrcProperty : InSrcModel->Properties)
	{
		const TSharedPtr<FWebAPICodeGenProperty>& DstProperty = FindOrAddProperty(SrcProperty->Name);
		DstProperty->FromWebAPI(SrcProperty);
		if(!Namespace.IsEmpty())
		{
			DstProperty->Namespace = Namespace;
			DstProperty->Metadata.FindOrAdd(TEXT("Namespace")) = Namespace;
		}
	}
}
