// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/WebAPICodeGenerator.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "IWebAPIEditorModule.h"
#include "Algo/Copy.h"
#include "Algo/ForEach.h"
#include "Algo/Unique.h"
#include "Async/Async.h"
#include "CodeGen/Dom/WebAPICodeGenClass.h"
#include "CodeGen/Dom/WebAPICodeGenEnum.h"
#include "CodeGen/Dom/WebAPICodeGenFile.h"
#include "CodeGen/Dom/WebAPICodeGenOperation.h"
#include "CodeGen/Dom/WebAPICodeGenSettings.h"
#include "CodeGen/Dom/WebAPICodeGenStruct.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPIType.h"
#include "Misc/Paths.h"
#include "UObject/StrongObjectPtr.h"
#include "WebAPIDefinition.h"

#define LOCTEXT_NAMESPACE "WebAPICodeGeneratorBase"

template <typename CodeGenType>
TSharedPtr<FWebAPICodeGenFile> MakeCodeGenFile(
	const TWeakObjectPtr<UWebAPIDefinition>& InDefinition,
	const TSharedPtr<CodeGenType>& InCodeGenObject,
	const FString& InBasePath,
	const FString& InRelativeFilePath,
	const TArray<FString>& InAdditionalIncludes = {})
{
	const TSharedPtr<FWebAPICodeGenFile>& CodeGenFile = MakeShared<FWebAPICodeGenFile>();
	FString RelativePath;
	FString FileName;
	FString FileType;
	FPaths::Split(InRelativeFilePath, RelativePath, FileName, FileType);

	if(InRelativeFilePath.EndsWith(TEXT("h")) && InCodeGenObject->Name.HasTypeInfo())
	{
		InCodeGenObject->Name.TypeInfo->IncludePaths.Add(InRelativeFilePath);
	}

	CodeGenFile->BaseFilePath = InBasePath;
	CodeGenFile->RelativeFilePath = RelativePath;
	CodeGenFile->FileName = FileName;
	CodeGenFile->FileType = FileType;
	CodeGenFile->Namespace = InDefinition->GetGeneratorSettings().GetNamespace();
	CodeGenFile->CopyrightNotice = InDefinition->GetCopyrightNotice();
	CodeGenFile->IncludePaths.Append(InAdditionalIncludes);
	CodeGenFile->AddItem(InCodeGenObject);

	return CodeGenFile;
}

template <typename CodeGenType>
static void MakeDeclDefn(
	const FString& InRelativePath,
	const FString& InName,
	const TSharedPtr<CodeGenType>& InCodeGenObject,
	const TWeakObjectPtr<UWebAPIDefinition>& InDefinition,
	TArray<TSharedPtr<FWebAPICodeGenFile>>& CodeGenFiles,
	const TWeakObjectPtr<UObject>& InSrcSchemaObject = {},
	const TArray<FString>& InAdditionalDeclIncludes = {},
	const TArray<FString>& InAdditionalDefnIncludes = {})
{
	// Declaration
	{
		const FString BasePath = FPaths::Combine(InDefinition->GetGeneratorSettings().TargetModule.GetPath(), TEXT("Public"));
		const FString RelativeFilePath = FPaths::Combine(InRelativePath, InName + TEXT(".h"));

		checkf(!CodeGenFiles.ContainsByPredicate([&BasePath, &RelativeFilePath](const TSharedPtr<FWebAPICodeGenFile>& InCodeGenFile)
		{
			return InCodeGenFile->BaseFilePath == BasePath
			&& InCodeGenFile->RelativeFilePath == RelativeFilePath;
		}), TEXT("CodeGenFiles already contains a file with the same path (\"%s\"), this shouldn't happen!"), *RelativeFilePath);

		TArray<FString> AdditionalIncludes = InAdditionalDeclIncludes;

		const TSharedPtr<FWebAPICodeGenFile> CodeGenFile = MakeCodeGenFile(InDefinition,
																		   InCodeGenObject,
																		   BasePath,
																		   RelativeFilePath,
																		   AdditionalIncludes);
		
		CodeGenFile->SchemaObjects.Add(InSrcSchemaObject);
		
		CodeGenFiles.Add(CodeGenFile);
	}

	// Definition
	{
		const FString BasePath = FPaths::Combine(InDefinition->GetGeneratorSettings().TargetModule.GetPath(), TEXT("Private"));
		const FString RelativeFilePath = FPaths::Combine(InRelativePath, InName + TEXT(".cpp"));

		checkf(!CodeGenFiles.ContainsByPredicate([&BasePath, &RelativeFilePath](const TSharedPtr<FWebAPICodeGenFile>& InCodeGenFile)
		{
			return InCodeGenFile->BaseFilePath == BasePath
			&& InCodeGenFile->RelativeFilePath == RelativeFilePath;
		}), TEXT("CodeGenFiles already contains a file with the same path (\"%s\"), this shouldn't happen!"), *RelativeFilePath);

		TArray<FString> AdditionalIncludes = InAdditionalDefnIncludes;

		const TSharedPtr<FWebAPICodeGenFile> CodeGenFile = MakeCodeGenFile(InDefinition,
																		   InCodeGenObject,
																		   BasePath,
																		   RelativeFilePath,
																		   AdditionalIncludes);

		CodeGenFile->SchemaObjects.Add(InSrcSchemaObject);
		
 		CodeGenFiles.Add(CodeGenFile);
	}

	const TSharedPtr<FWebAPIMessageLog> MessageLog = InDefinition->GetMessageLog();
				
	FFormatNamedArguments Args;
	Args.Add(TEXT("Name"), FText::FromString(InName));
	MessageLog->LogInfo(FText::Format(LOCTEXT("FilePrepared", "A file h/cpp pair called \"{Name}\" has been queued for generation."), Args), UWebAPICodeGeneratorBase::LogName);
}

template <typename CodeGenType>
bool UWebAPICodeGeneratorBase::CheckNameConflicts(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TArray<TSharedPtr<CodeGenType>>& InItems) const
{
	static_assert(std::is_base_of_v<FWebAPICodeGenBase, CodeGenType>, "CodeGenType must be derived from FWebAPICodeGenBase");
	
	check(InDefinition.IsValid());
	
	TArray<TSharedPtr<FWebAPICodeGenBase>> Items = InItems;	
	const int32 UniqueItemCount = Algo::Unique(Items,
		[&](TSharedPtr<FWebAPICodeGenBase> A, TSharedPtr<FWebAPICodeGenBase> B)
		{
			return A->GetName(true).Equals(B->GetName(true));
		});

	// Find and report actual duplicates
	if(UniqueItemCount != InItems.Num())
	{
		TArray<TSharedPtr<FWebAPICodeGenBase>> SameItems;
		for(const TSharedPtr<FWebAPICodeGenBase>& Item : InItems)
		{
			SameItems.Empty(SameItems.Num());
			Algo::CopyIf(InItems, SameItems, [&](const TSharedPtr<FWebAPICodeGenBase>& InItemInner)
			{
				return InItemInner->GetName(true).Equals(Item->GetName(true));	 
			});

			if(SameItems.Num() > 1)
			{
				const TSharedPtr<FWebAPIMessageLog> MessageLog = InDefinition->GetMessageLog();
				
				FFormatNamedArguments Args;
				Args.Add(TEXT("Name"), FText::FromString(Item->GetName(true)));
				MessageLog->LogError(FText::Format(LOCTEXT("DuplicateTypesFound", "One or more duplicates found with the name \"{Name}\"."), Args), UWebAPICodeGeneratorBase::LogName);
			}
		}
		return false;
	}

 	return true;
}

template <>
bool UWebAPICodeGeneratorBase::CheckNameConflicts<FWebAPICodeGenFile>(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TArray<TSharedPtr<FWebAPICodeGenFile>>& InItems) const
{
	check(InDefinition.IsValid());
	
	TArray<TSharedPtr<FWebAPICodeGenFile>> Items = InItems;
	Items.StableSort([&](const TSharedPtr<FWebAPICodeGenFile>& A, const TSharedPtr<FWebAPICodeGenFile>& B)
	{
		return (A->FileName + A->FileType) < (B->FileName + B->FileType);
	});
	
	const int32 UniqueItemCount = Algo::Unique(Items,
		[&](TSharedPtr<FWebAPICodeGenFile> A, TSharedPtr<FWebAPICodeGenFile> B)
		{
			return(A->FileName + A->FileType) == (B->FileName + B->FileType);
		});

	// Find and report actual duplicates
	if(UniqueItemCount != InItems.Num())
	{
		TArray<TSharedPtr<FWebAPICodeGenFile>> SameItems;
		for(const TSharedPtr<FWebAPICodeGenFile>& Item : InItems)
		{
			SameItems.Empty(SameItems.Num());
			Algo::CopyIf(InItems, SameItems, [&](const TSharedPtr<FWebAPICodeGenFile>& InItemInner)
			{
				return (InItemInner->FileName + InItemInner->FileType) == (Item->FileName + Item->FileType);
			});

			if(SameItems.Num() > 1)
			{
				const TSharedPtr<FWebAPIMessageLog> MessageLog = InDefinition->GetMessageLog();
				
				FFormatNamedArguments Args;
				Args.Add(TEXT("Name"), FText::FromString(Item->FileName));
				MessageLog->LogError(FText::Format(LOCTEXT("DuplicateFilesFound", "One or more duplicate file output paths found with the filename \"{Name}\"."), Args), UWebAPICodeGeneratorBase::LogName);
			}
		}
		return false;
	}

	return true;
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::Generate(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition)
{
	const TSharedRef<TPromise<EWebAPIGenerationResult>, ESPMode::ThreadSafe> Promise = MakeShared<TPromise<EWebAPIGenerationResult>, ESPMode::ThreadSafe>();

	AsyncTask(ENamedThreads::GameThread, [&, Promise, Definition = InDefinition]
	{
		if (!Definition.IsValid())
		{
			Promise->SetValue(EWebAPIGenerationResult::Failed);
			return;
		}

		TStrongObjectPtr<UWebAPIDefinition> DefinitionPtr = TStrongObjectPtr<UWebAPIDefinition>(Definition.Get());
		
		const TObjectPtr<UWebAPISchema> WebAPISchema = Definition->GetWebAPISchema();
		TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
		
		TArray<TSharedPtr<FWebAPICodeGenEnum>> CodeGenEnums;
		TMap<TSharedPtr<FWebAPICodeGenEnum>, TWeakObjectPtr<UObject>> CodeGenEnumSchemas;
		
		TArray<TSharedPtr<FWebAPICodeGenStruct>> CodeGenStructs;
		TMap<TSharedPtr<FWebAPICodeGenStruct>, TWeakObjectPtr<UObject>> CodeGenStructSchemas;
		
		TArray<TSharedPtr<FWebAPICodeGenClass>> CodeGenClasses;
		TMap<TSharedPtr<FWebAPICodeGenClass>, TWeakObjectPtr<UObject>> CodeGenClassSchemas;
		
		TMap<TObjectPtr<UWebAPIService>, TArray<TSharedPtr<FWebAPICodeGenOperation>>> CodeGenOperations;
		TMap<TSharedPtr<FWebAPICodeGenOperation>, TWeakObjectPtr<UObject>> CodeGenOperationSchemas;

		// Convert WebAPI schema to CodeGen objects
		{
			for (const TObjectPtr<UWebAPIModelBase>& Model : WebAPISchema->Models)
			{
				if (UWebAPIParameter* SrcParameter = Cast<UWebAPIParameter>(Model))
				{
					const TSharedPtr<FWebAPICodeGenStruct> DstStruct = MakeShared<FWebAPICodeGenStruct>();
					DstStruct->FromWebAPI(SrcParameter);
					CodeGenStructs.Add(DstStruct);
					CodeGenStructSchemas.Add(DstStruct, SrcParameter);
				}
				else if (UWebAPIModel* SrcModel = Cast<UWebAPIModel>(Model))
				{
					if(!SrcModel->bGenerate)
					{
						continue;
					}
					
					const TSharedPtr<FWebAPICodeGenStruct> DstStruct = MakeShared<FWebAPICodeGenStruct>();
					DstStruct->FromWebAPI(SrcModel);

					// If there are recursive properties (property type == struct type), make it a class/uobject instead of struct
					TArray<TSharedPtr<FWebAPICodeGenProperty>> RecursiveProperties;
					if(DstStruct->FindRecursiveProperties(RecursiveProperties))
					{
						for(TSharedPtr<FWebAPICodeGenProperty>& Property : RecursiveProperties)
						{
							if(Property->Type.HasTypeInfo())
							{
								// UObjects have "U" prefix rather than "F" for struct
								Property->Type.TypeInfo->Prefix = TEXT("U");								
							}
						}
						const TSharedPtr<FWebAPICodeGenClass> DstClass = MakeShared<FWebAPICodeGenClass>(*DstStruct.Get());
 						CodeGenClasses.Add(DstClass);
						CodeGenClassSchemas.Add(DstClass, SrcModel);
					}
					else
					{
						CodeGenStructs.Add(DstStruct);
						CodeGenStructSchemas.Add(DstStruct, SrcModel);
					}
				}
				else if (UWebAPIEnum* SrcEnum = Cast<UWebAPIEnum>(Model))
				{
					if(!SrcEnum->bGenerate)
					{
						continue;
					}
					
					const TSharedPtr<FWebAPICodeGenEnum> DstEnum = MakeShared<FWebAPICodeGenEnum>();
					DstEnum->FromWebAPI(SrcEnum);
					CodeGenEnums.Add(DstEnum);
					CodeGenEnumSchemas.Add(DstEnum, SrcEnum);
				}
			}

			for (const TTuple<FString, TObjectPtr<UWebAPIService>>& Service : WebAPISchema->Services)
			{
				if(!Service.Value->bGenerate)
				{
					continue;
				}
				
				FString ServiceName = Service.Key;
				TArray<TSharedPtr<FWebAPICodeGenOperation>> ServiceOperations;
				for (TObjectPtr<UWebAPIOperation>& SrcOperation : Service.Value->Operations)
				{
					if(!SrcOperation->bGenerate)
					{
						continue;
					}
					
					const TSharedPtr<FWebAPICodeGenOperation> DstOperation = MakeShared<FWebAPICodeGenOperation>();
					DstOperation->FromWebAPI(SrcOperation);
					ServiceOperations.Add(DstOperation);
					CodeGenOperationSchemas.Add(DstOperation, SrcOperation);
				}
				CodeGenOperations.Add(Service.Value, MoveTemp(ServiceOperations));				
			}
		}

		TArray<TSharedPtr<FWebAPICodeGenBase>> CodeGenObjects;
		CodeGenObjects.Append(CodeGenEnums);
		CodeGenObjects.Append(CodeGenStructs);
		CodeGenObjects.Append(CodeGenClasses);
		
		TArray<TSharedPtr<FWebAPICodeGenOperation>> CodeGenOperationsOnly;
		Algo::ForEach(CodeGenOperations, [&](const TPair<TObjectPtr<UWebAPIService>, TArray<TSharedPtr<FWebAPICodeGenOperation>>>& InServiceOperationPair)
		{
			CodeGenOperationsOnly.Append(InServiceOperationPair.Value);
		});
		CodeGenObjects.Append(CodeGenOperationsOnly);

		TSet<TSharedPtr<FWebAPICodeGenBase>> CodeGenObjectsSet(CodeGenObjects);
		if(!CheckNameConflicts(Definition, CodeGenObjects))
		{
			Promise->SetValue(EWebAPIGenerationResult::FailedWithErrors);
			return;
		}

		// Convert CodeGen objects to Definition/Declarations
		FString SettingsTypeName = TEXT("");
		TArray<TSharedPtr<FWebAPICodeGenFile>> CodeGenFiles;
		{
			for(const TSharedPtr<FWebAPICodeGenStruct>& CodeGenStruct : CodeGenStructs)
			{
				FStringFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("Model"), CodeGenStruct->Name.TypeInfo->ToMemberName());
				FString RelativeFilePath = FString::Format(*Definition->GetGeneratorSettings().ModelOutputPath, FormatArgs);
				MakeDeclDefn(RelativeFilePath, CodeGenStruct->Name.TypeInfo->ToMemberName(), CodeGenStruct, Definition, CodeGenFiles, CodeGenStructSchemas[CodeGenStruct],
				{							
					TEXT("Dom/JsonObject.h"),
					TEXT("Dom/JsonValue.h")
				});
			}

			for(const TSharedPtr<FWebAPICodeGenEnum>& CodeGenEnum : CodeGenEnums)
			{
				FStringFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("Model"), CodeGenEnum->Name.TypeInfo->ToMemberName());
				FString RelativeFilePath = FString::Format(*Definition->GetGeneratorSettings().ModelOutputPath, FormatArgs);
				MakeDeclDefn(RelativeFilePath, CodeGenEnum->Name.TypeInfo->ToMemberName(), CodeGenEnum, Definition, CodeGenFiles, CodeGenEnumSchemas[CodeGenEnum],
					{
						TEXT("Dom/JsonObject.h"),
						TEXT("Dom/JsonValue.h")
					});
			}

			for(const TSharedPtr<FWebAPICodeGenClass>& CodeGenClass : CodeGenClasses)
			{
				FStringFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("Model"), CodeGenClass->Name.TypeInfo->ToMemberName());
				FString RelativeFilePath = FString::Format(*Definition->GetGeneratorSettings().ModelOutputPath, FormatArgs);
				MakeDeclDefn(RelativeFilePath, CodeGenClass->Name.TypeInfo->ToMemberName(), CodeGenClass, Definition, CodeGenFiles, CodeGenClassSchemas[CodeGenClass],
					{
						TEXT("Dom/JsonObject.h"),
						TEXT("Dom/JsonValue.h")
					});
			}

			{
				const TSharedPtr<FWebAPICodeGenSettings> Settings = MakeShared<FWebAPICodeGenSettings>();
				Settings->FromWebAPI(Definition.Get());
				SettingsTypeName = Settings->Name.ToString();

				MakeDeclDefn(TEXT(""), Settings->Name.TypeInfo->ToMemberName(), Settings, Definition, CodeGenFiles);	
			}

			for(const TPair<TObjectPtr<UWebAPIService>, TArray<TSharedPtr<FWebAPICodeGenOperation>>>& ServiceOperations : CodeGenOperations)
			{
				for(const TSharedPtr<FWebAPICodeGenOperation>& CodeGenOperation : ServiceOperations.Value)
				{
					CodeGenOperation->SettingsTypeName = SettingsTypeName;

					FStringFormatNamedArguments FormatArgs;
					FormatArgs.Add(TEXT("Service"), ServiceOperations.Key->Name.TypeInfo->ToMemberName());
					FormatArgs.Add(TEXT("Operation"), CodeGenOperation->Name.TypeInfo->ToMemberName());
					FString RelativeFilePath = FString::Format(*Definition->GetGeneratorSettings().OperationOutputPath, FormatArgs);
					MakeDeclDefn(RelativeFilePath, CodeGenOperation->Name.TypeInfo->ToMemberName(), CodeGenOperation, Definition, CodeGenFiles, CodeGenOperationSchemas[CodeGenOperation],
						{
							TEXT("CoreMinimal.h"),
							TEXT("WebAPIOperationObject.h"),
						},
						{
							TEXT("HttpModule.h"),
							TEXT("Interfaces/IHttpResponse.h"),
							TEXT("Serialization/JsonReader.h"),
							TEXT("Serialization/JsonSerializer.h"),
							TEXT("WebAPISubsystem.h"),								
							SettingsTypeName.RightChop(1) + TEXT(".h")
						});
				}
			}
		}

		// Check file outputs for duplicates
		TSet<TSharedPtr<FWebAPICodeGenFile>> CodeGenFilesSet(CodeGenFiles);
		if(!CheckNameConflicts(Definition, CodeGenFiles))
		{
			Promise->SetValue(EWebAPIGenerationResult::FailedWithErrors);
			return;
		}

		// Add module name
		for(TSharedPtr<FWebAPICodeGenFile>& CodeGenFile : CodeGenFiles)
		{
			CodeGenFile->SetModule(Definition->GetGeneratorSettings().TargetModule.Name);
		}

		// Re-gather includes
		for(TSharedPtr<FWebAPICodeGenFile>& CodeGenFile : CodeGenFiles)
		{
			CodeGenFile->GetIncludePaths(CodeGenFile->IncludePaths);

			if(CodeGenFile->FileType == TEXT("h"))
			{
				const FString RelativeFilePath = FPaths::Combine(CodeGenFile->RelativeFilePath, CodeGenFile->FileName + TEXT(".") + CodeGenFile->FileType);
				CodeGenFile->IncludePaths.Remove(RelativeFilePath);
			}
		}
		
		Async(EAsyncExecution::Thread,
		[&, Promise, Definition = DefinitionPtr.Get(), CodeGenFiles]
		{
			// Clear (debug) CodeText on original schema objects
			{
				Algo::ForEach(WebAPISchema->Models, [&](const TObjectPtr<UWebAPIModelBase>& InModel)
				{
					InModel->SetCodeText(TEXT(""));
				});

				Algo::ForEach(WebAPISchema->Services, [&](const TPair<FString, TObjectPtr<UWebAPIService>>& InServiceOperationPair)
				{
					Algo::ForEach(InServiceOperationPair.Value->Operations, [&](const TObjectPtr<UWebAPIOperation>& InOperation)
					{
						InOperation->SetCodeText(TEXT(""));
					});
				});
			}

			// Generate actual files
			for(const TSharedPtr<FWebAPICodeGenFile>& CodeGenFile : CodeGenFiles)
			{
				GenerateFile(Definition, CodeGenFile);
			}
			
			Promise->SetValue(EWebAPIGenerationResult::Succeeded);
		});
	});

	return Promise->GetFuture(); 
}

TFuture<EWebAPIGenerationResult> IWebAPICodeGeneratorInterface::Generate(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

TFuture<bool> UWebAPICodeGeneratorBase::IsAvailable()
{
	return MakeFulfilledPromise<bool>(false).GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::GenerateServiceOperations(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const FString& InServiceName, const TArray<TSharedPtr<FWebAPICodeGenOperation>>& InOperations)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::GenerateEnum(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenEnum>& InEnum)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::GenerateModel(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenStruct>& InStruct)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::GenerateFile(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenFile>& InFile)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPICodeGeneratorBase::GenerateSettings(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenClass>& InSettingsClass)
{
	return MakeFulfilledPromise<EWebAPIGenerationResult>().GetFuture();
}

#undef LOCTEXT_NAMESPACE
