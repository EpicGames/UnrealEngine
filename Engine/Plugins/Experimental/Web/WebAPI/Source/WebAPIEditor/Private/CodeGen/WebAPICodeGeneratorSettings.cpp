// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGen/WebAPICodeGeneratorSettings.h"

#include "CodeGen/WebAPICodeGenerator.h"
#include "GeneralProjectSettings.h"
#include "WebAPIEditorSettings.h"

FWebAPICodeGeneratorSettings::FWebAPICodeGeneratorSettings()
{
	CopyrightNotice = GetDefault<UGeneralProjectSettings>()->CopyrightNotice;
}

TScriptInterface<IWebAPICodeGeneratorInterface> FWebAPICodeGeneratorSettings::GetGeneratorClass() const
{
	if(!CodeGeneratorClass.IsNull() && bOverrideGeneratorClass)
	{
		const TScriptInterface<IWebAPICodeGeneratorInterface> CodeGeneratorInstance = CodeGeneratorClass->GetDefaultObject();
		return CodeGeneratorInstance;		
	}

	const UWebAPIEditorSettings* ProjectSettings = GetDefault<UWebAPIEditorSettings>();
	return ProjectSettings->GetGeneratorClass();
}

const FString& FWebAPICodeGeneratorSettings::GetNamespace() const
{
	return Namespace;
}

void FWebAPICodeGeneratorSettings::SetNamespace(const FString& InNamespace)
{
	if(InNamespace == Namespace)
	{
		return;
	}

	Namespace = InNamespace;
	OnNamespaceChanged().Broadcast(Namespace);
}

FOnNamespaceChangedDelegate& FWebAPICodeGeneratorSettings::OnNamespaceChanged()
{
	return OnNamespaceChangedDelegate;
}
