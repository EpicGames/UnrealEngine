// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPICodeViewModel.h"

#include "Details/ViewModels/WebAPIViewModel.h"
#include "WebAPIDefinition.h"

#define LOCTEXT_NAMESPACE "WebAPICodeViewModel"

TSharedRef<FWebAPICodeViewModel> FWebAPICodeViewModel::Create(UWebAPIDefinition* InDefinition)
{
	TSharedRef<FWebAPICodeViewModel> ViewModel = MakeShared<FWebAPICodeViewModel>(FPrivateToken{}, InDefinition);
	ViewModel->Initialize();

	return ViewModel;
}

FWebAPICodeViewModel::FWebAPICodeViewModel(FPrivateToken)
{
	CodeContents = MakeShared<FText>();
}

FWebAPICodeViewModel::FWebAPICodeViewModel(FPrivateToken, UWebAPIDefinition* InDefinition)
	: Definition(InDefinition)
	, CodeContents(MakeShared<FText>())
{
	check(Definition.IsValid());
}

void FWebAPICodeViewModel::Initialize()
{
}

FText FWebAPICodeViewModel::GetCodeText() const
{
	return *CodeContents;
}

void FWebAPICodeViewModel::SetCodeText(const FText& InCodeText) const
{
	*CodeContents = InCodeText;
}

bool FWebAPICodeViewModel::IsValid() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
