// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebAPIViewModel.h"

class FWebAPICodeViewModel
	: public TSharedFromThis<FWebAPICodeViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPICodeViewModel> Create(UWebAPIDefinition* InDefinition);
	virtual ~FWebAPICodeViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override { return nullptr; }
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override { return false; }
	virtual int32 GetIndex() const override { return 0; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Code; }

	const TWeakObjectPtr<UWebAPIDefinition>& GetDefinition() const
	{ return Definition; }
	FText GetCodeText() const;
	void SetCodeText(const FText& InCodeText) const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override;

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPICodeViewModel(FPrivateToken);
	FWebAPICodeViewModel(FPrivateToken, UWebAPIDefinition* InDefinition);

private:
	void Initialize();

private:
	TWeakObjectPtr<UWebAPIDefinition> Definition;
	TSharedPtr<FText> CodeContents;
};
