// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEnumViewModel.h"

#include "Dom/WebAPIType.h"

#define LOCTEXT_NAMESPACE "WebAPIEnumViewModel"

TSharedRef<FWebAPIEnumValueViewModel> FWebAPIEnumValueViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnumValue* InValue, int32 InIndex)
{
	TSharedRef<FWebAPIEnumValueViewModel> ViewModel = MakeShared<FWebAPIEnumValueViewModel>(FPrivateToken{}, InParentViewModel, InValue, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

FWebAPIEnumValueViewModel::FWebAPIEnumValueViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnumValue* InValue, int32 InIndex)
	: Enum(InParentViewModel)
	, Value(InValue)
	, Index(InIndex)
{
	check(Value.IsValid());
}

void FWebAPIEnumValueViewModel::Initialize()
{
	CachedLabel = FText::FromString(Value->Name.GetDisplayName());

	const FString Prefix = TEXT("EV");
	
	TArray<FText> TooltipArgs;
	TooltipArgs.Emplace(LOCTEXT("EnumValueTooltip_Header", "Enum Value").ToUpper());
	TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumValueTooltip_MemberName", "MemberName: {0}"), FText::FromString(Value->Name.NameInfo.ToMemberName(Prefix))));
	
	if(!Value->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumValueTooltip_Description", "Description: {0}"), FText::FromString(Value->Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumValueTooltip_JsonName", "JsonName: {0}"), FText::FromString(Value->Name.GetJsonName())));

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

TSharedPtr<IWebAPIViewModel> FWebAPIEnumValueViewModel::GetParent()
{
	return Enum;
}

TSharedRef<FWebAPIEnumViewModel> FWebAPIEnumViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnum* InEnum, int32 InIndex)
{
	TSharedRef<FWebAPIEnumViewModel> ViewModel = MakeShared<FWebAPIEnumViewModel>(FPrivateToken{}, InParentViewModel, InEnum, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

FWebAPIEnumViewModel::FWebAPIEnumViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnum* InEnum, int32 InIndex)
	: Schema(InParentViewModel)
	, Enum(InEnum)
	, Index(InIndex)
{
	check(Enum.IsValid());
}

void FWebAPIEnumViewModel::Initialize()
{
	Values.Empty(Values.Num());
	for(const TObjectPtr<UWebAPIEnumValue>& Value : Enum->Values)
	{
		if(Value)
		{
			TSharedRef<FWebAPIEnumValueViewModel> EnumValueViewModel = FWebAPIEnumValueViewModel::Create(AsShared(), Value);
			Values.Add(MoveTemp(EnumValueViewModel));			
		}
	}

	CachedLabel = FText::FromString(Enum->Name.GetDisplayName());

	TArray<FText> TooltipArgs;

	if(Enum->Name.HasTypeInfo() && Enum->Name.TypeInfo->bIsNested)
	{
		TooltipArgs.Emplace(LOCTEXT("NestedEnumTooltip_Header", "Enum (Nested)").ToUpper());
		TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumTooltip_MemberName", "MemberName: {0}"), FText::FromString(Enum->Name.TypeInfo->ToMemberName())));
	}
	else
	{
		TooltipArgs.Emplace(LOCTEXT("EnumTooltip_Header", "Enum").ToUpper());
	}

	if(!Enum->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumTooltip_Description", "Description: {0}"), FText::FromString(Enum->Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumTooltip_JsonName", "JsonName: {0}"), FText::FromString(Enum->Name.GetJsonName())));
	TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumTooltip_Type", "Type: {0}"), FText::FromString(Enum->Type)));

	const FString DefaultValue = Enum->GetDefaultValue();
	if(!DefaultValue.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("EnumTooltip_DefaultValue", "Default: {0}"), FText::FromString(DefaultValue)));
	}

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

TSharedPtr<IWebAPIViewModel> FWebAPIEnumViewModel::GetParent()
{
	return Schema;
}

bool FWebAPIEnumViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Values);	
	return !Values.IsEmpty();
}

bool FWebAPIEnumViewModel::GetShouldGenerate() const
{
	return Enum.IsValid() && Enum->bGenerate;
}

#undef LOCTEXT_NAMESPACE
