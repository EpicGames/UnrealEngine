// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"
#include "Dom/WebAPIEnum.h"

class FWebAPIEnumViewModel;

class FWebAPIEnumValueViewModel
	: public TSharedFromThis<FWebAPIEnumValueViewModel>
	, public IWebAPIViewModel	
{
public:
	static TSharedRef<FWebAPIEnumValueViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnumValue* InValue, int32 InIndex = -1);

	virtual ~FWebAPIEnumValueViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override { return false; }
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_EnumValue; }

	FString GetValue() const { return Value->Name.GetDisplayName(); }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return !Value.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIEnumValueViewModel(FPrivateToken) {}
	FWebAPIEnumValueViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnumValue* InValue, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<IWebAPIViewModel> Enum;
	TWeakObjectPtr<UWebAPIEnumValue> Value;
	int32 Index = 0;
};

class FWebAPIEnumViewModel
	: public TSharedFromThis<FWebAPIEnumViewModel>
	, public IWebAPIViewModel	
{
public:
	static TSharedRef<FWebAPIEnumViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnum* InEnum, int32 InIndex = -1);
	virtual ~FWebAPIEnumViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Enum; }

	bool GetShouldGenerate() const;
	const TArray<TSharedPtr<FWebAPIEnumValueViewModel>>& GetValues() const { return Values; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Enum.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIEnumViewModel(FPrivateToken) {}
	FWebAPIEnumViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIEnum* InEnum, int32 InIndex = -1);

private:
	void Initialize();

private:
	friend class FWebAPIEnumValueViewModel;
	
	TSharedPtr<IWebAPIViewModel> Schema;
	TWeakObjectPtr<UWebAPIEnum> Enum;
	TArray<TSharedPtr<FWebAPIEnumValueViewModel>> Values;
	int32 Index = 0;
};
