// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIModelViewModel.h"

#include "Details/ViewModels/WebAPIEnumViewModel.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "WebAPIViewModel.inl"
#include "Details/ViewModels/WebAPIOperationViewModel.h"
#include "Dom/WebAPIType.h"
#include "Details/ViewModels/WebAPIParameterViewModel.h"
#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"

#define LOCTEXT_NAMESPACE "WebAPIModelViewModel"

TSharedRef<FWebAPIPropertyViewModel> FWebAPIPropertyViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIProperty* InProperty, int32 InIndex)
{
	TSharedRef<FWebAPIPropertyViewModel> ViewModel = MakeShared<FWebAPIPropertyViewModel>(FPrivateToken{}, InParentViewModel, InProperty, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIPropertyViewModel::GetParent()
{
	return Model;
}

bool FWebAPIPropertyViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	if(NestedModel.IsValid())
	{
		return NestedModel->GetChildren(OutChildren);
	}
	return false;
}

bool FWebAPIPropertyViewModel::HasCodeText() const
{
	if(!IsValid())
	{
		return IWebAPIViewModel::HasCodeText();
	}

	if(Property->Type.HasTypeInfo())
	{
		if(const UObject* PropertyModel = Property->Type.TypeInfo->GetModel())
		{
			if(const UWebAPIModel* AsModel = Cast<UWebAPIModel>(PropertyModel))
			{
				return !AsModel->GeneratedCodeText.IsEmpty();
			}
			else if(const UWebAPIEnum* AsEnum = Cast<UWebAPIEnum>(PropertyModel))
			{
				return !AsEnum->GeneratedCodeText.IsEmpty();
			}
		}
	}

	return false;
}

FText FWebAPIPropertyViewModel::GetCodeText() const
{
	if(!IsValid())
	{
		return IWebAPIViewModel::GetCodeText();
	}

	if(Property->Type.HasTypeInfo())
	{
		if(const UObject* PropertyModel = Property->Type.TypeInfo->GetModel())
		{
			if(const UWebAPIModel* AsModel = Cast<UWebAPIModel>(PropertyModel))
			{
				return FText::FromString(AsModel->GeneratedCodeText);
			}
			else if(const UWebAPIEnum* AsEnum = Cast<UWebAPIEnum>(PropertyModel))
			{
				return FText::FromString(AsEnum->GeneratedCodeText);
			}
		}
	}
 
	return IWebAPIViewModel::GetCodeText();
}

FSlateColor FWebAPIPropertyViewModel::GetPinColor() const
{
	if(Property.IsValid() && Property->Type.HasTypeInfo())
	{ 
		return Property->Type.TypeInfo->PinColor;		
	}

	return FSlateColor(FLinearColor(0,0,0));
}

bool FWebAPIPropertyViewModel::IsArray() const
{ 
	if(Property.IsValid())
	{
		return Property->bIsArray;
	}
	return false;
}

FWebAPIPropertyViewModel::FWebAPIPropertyViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIProperty* InProperty, int32 InIndex)
	: Model(InParentViewModel)
	, Property(InProperty)
	, Index(InIndex)
{
}

void FWebAPIPropertyViewModel::Initialize()
{
	const FString Name = Property->Name.ToString();
	const FString Description = Property->Description;
	const FWebAPITypeNameVariant Type = Property->Type;

	if(Type.HasTypeInfo() && Type.TypeInfo->bIsNested)
	{
		if(UObject* PropertyModel = Type.TypeInfo->GetModel())
		{
			NestedModel = UE::WebAPI::Details::CreateViewModel(SharedThis(this), PropertyModel);
		}
	}

	FString DebugLabel;
#if WITH_WEBAPI_DEBUG
	bool bIsTypeResolved = true;
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		bIsTypeResolved = Type.TypeInfo->GetModel() != nullptr;
	}

	if(!bIsTypeResolved)
	{
		DebugLabel = TEXT(" (Unresolved)");
	}
#endif

	CachedLabel = FText::Format(LOCTEXT("PropertyLabel", "{0}: {1}"),
		FText::FromString(Name),
		FText::FromString(Type.GetDisplayName() + (IsArray() ? TEXT("[]") : TEXT("")) + DebugLabel));

	TArray<FText> TooltipArgs;
	TooltipArgs.Emplace(LOCTEXT("PropertyTooltip_Header", "Property").ToUpper());
	TooltipArgs.Emplace(FText::Format(LOCTEXT("PropertyTooltip_MemberName", "MemberName: {0}"), FText::FromString(Property->Name.ToMemberName())));
	
	if(!Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("PropertyTooltip_Description", "Description: {0}"), FText::FromString(Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("PropertyTooltip_JsonName", "JsonName: {0}"), FText::FromString(Property->Name.GetJsonName())));
	TooltipArgs.Emplace(FText::Format(LOCTEXT("PropertyTooltip_Type", "Type: {0}"),	FText::FromString(Type.ToString())));

	const FString DefaultValue = Property->GetDefaultValue();
	if(!DefaultValue.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("PropertyTooltip_DefaultValue", "Default: {0}"), FText::FromString(DefaultValue)));
	}

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

TSharedRef<FWebAPIModelViewModel> FWebAPIModelViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIModel* InModel, int32 InIndex)
{
	TSharedRef<FWebAPIModelViewModel> ViewModel = MakeShared<FWebAPIModelViewModel>(FPrivateToken{}, InParentViewModel, InModel, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

FWebAPIModelViewModel::FWebAPIModelViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIModel* InModel, int32 InIndex)
	: Schema(InParentViewModel)
	, Model(InModel)
	, Index(InIndex)
{
	check(Model.IsValid());
}

void FWebAPIModelViewModel::Initialize()
{
	Properties.Empty(Properties.Num());
	for(const TObjectPtr<UWebAPIProperty>& Property : Model->Properties)
	{
		TSharedRef<FWebAPIPropertyViewModel> PropertyViewModel = FWebAPIPropertyViewModel::Create(AsShared(), Property);
		Properties.Add(MoveTemp(PropertyViewModel));
	}

	CachedLabel = FText::FromString(Model->Name.GetDisplayName());

	TArray<FText> TooltipArgs;
	if(Model->Name.HasTypeInfo() && Model->Name.TypeInfo.LoadSynchronous()->bIsNested)
	{
		TooltipArgs.Emplace(LOCTEXT("NestedModelTooltip_Header", "Model (Nested)").ToUpper());
	}
	else
	{
		TooltipArgs.Emplace(LOCTEXT("ModelTooltip_Header", "Model").ToUpper());
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("ModelTooltip_MemberName", "MemberName: {0}"), FText::FromString(Model->Name.ToMemberName())));

#if WITH_WEBAPI_DEBUG
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ModelTooltip_FullName", "FullName: {0}"), FText::FromString(Model->Name.ToString())));
#endif
	
	if(!Model->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("ModelTooltip_Description", "Description: {0}"), FText::FromString(Model->Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("ModelTooltip_JsonName", "JsonName: {0}"), FText::FromString(Model->Name.GetJsonName())));
	
	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

TSharedPtr<IWebAPIViewModel> FWebAPIModelViewModel::GetParent()
{
	return Schema;
}

bool FWebAPIModelViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Properties);
	return !Properties.IsEmpty();
}

bool FWebAPIModelViewModel::HasCodeText() const
{
	return IsValid() && !Model->GeneratedCodeText.IsEmpty();
}

FText FWebAPIModelViewModel::GetCodeText() const
{
	if(!IsValid())
	{
		return IWebAPIViewModel::GetCodeText();
	}

	return FText::FromString(Model->GeneratedCodeText);
}

bool FWebAPIModelViewModel::GetShouldGenerate() const
{
	return Model.IsValid() && Model->bGenerate;
}

#undef LOCTEXT_NAMESPACE
