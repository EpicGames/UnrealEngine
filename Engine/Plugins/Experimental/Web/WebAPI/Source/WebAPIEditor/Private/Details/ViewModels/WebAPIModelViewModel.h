// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class UWebAPIModel;
class UWebAPIProperty;
struct FSlateColor;

class FWebAPIPropertyViewModel
	: public TSharedFromThis<FWebAPIPropertyViewModel>
	, public IWebAPIViewModel	
{
public:
	static TSharedRef<FWebAPIPropertyViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIProperty* InProperty, int32 InIndex = -1);
	virtual ~FWebAPIPropertyViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Property; }
	virtual bool HasCodeText() const override;
	virtual FText GetCodeText() const override;

	const TSharedPtr<IWebAPIViewModel>& GetModel() const { return Model; }
	FSlateColor GetPinColor() const;

	bool IsArray() const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Property.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	FWebAPIPropertyViewModel(FPrivateToken) {}
	FWebAPIPropertyViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIProperty* InProperty, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<IWebAPIViewModel> Model;
	TWeakObjectPtr<UWebAPIProperty> Property;

	/** Optional nested model to display, if this properties type had to be uniquely generated and has two or more members. */
	TSharedPtr<IWebAPIViewModel> NestedModel;
	
	int32 Index = 0;
};

class FWebAPIModelViewModel
	: public TSharedFromThis<FWebAPIModelViewModel>
	, public IWebAPIViewModel	
{
public:
	static TSharedRef<FWebAPIModelViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIModel* InModel, int32 InIndex = -1);
	virtual ~FWebAPIModelViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Model; }
	virtual bool HasCodeText() const override;
	virtual FText GetCodeText() const override;

	bool GetShouldGenerate() const;
	const TArray<TSharedPtr<FWebAPIPropertyViewModel>>& GetProperties() const { return Properties; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Model.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIModelViewModel(FPrivateToken) {}
	FWebAPIModelViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIModel* InModel, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<IWebAPIViewModel> Schema;
	TWeakObjectPtr<UWebAPIModel> Model;
	TArray<TSharedPtr<FWebAPIPropertyViewModel>> Properties;
	int32 Index = 0;
};
