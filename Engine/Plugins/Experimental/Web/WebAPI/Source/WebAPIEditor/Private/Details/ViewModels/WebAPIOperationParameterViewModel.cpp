// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOperationParameterViewModel.h"

#include "WebAPIOperationRequestViewModel.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIType.h"

#define LOCTEXT_NAMESPACE "WebAPIOperationParameterViewModel"

TSharedRef<FWebAPIOperationParameterViewModel> FWebAPIOperationParameterViewModel::Create(const TSharedRef<FWebAPIOperationRequestViewModel>& InParentViewModel, UWebAPIOperationParameter* InParameter, int32 InIndex)
{
	TSharedRef<FWebAPIOperationParameterViewModel> ViewModel = MakeShared<FWebAPIOperationParameterViewModel>(FPrivateToken{}, InParentViewModel, InParameter, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIOperationParameterViewModel::GetParent()
{
	return Request;
}

bool FWebAPIOperationParameterViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	return false;
}

FSlateColor FWebAPIOperationParameterViewModel::GetPinColor() const
{
	if(Parameter.IsValid() && Parameter->Type.HasTypeInfo())
	{ 
		return Parameter->Type.TypeInfo->PinColor;		
	}

	return FSlateColor(FLinearColor(0,0,0));
}

bool FWebAPIOperationParameterViewModel::IsArray() const
{
	return Parameter.IsValid() && Parameter->bIsArray;
}

FWebAPIOperationParameterViewModel::FWebAPIOperationParameterViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationRequestViewModel>& InParentViewModel, UWebAPIOperationParameter* InParameter, int32 InIndex)
	: Request(InParentViewModel)
	, Parameter(InParameter)
	, Index(InIndex)
{
}

void FWebAPIOperationParameterViewModel::Initialize()
{
	const FWebAPINameVariant Name = Parameter->Name;
	const FString Description = Parameter->Description;
	const FWebAPITypeNameVariant Type = Parameter->Type;

	FString DebugLabel;
#if WITH_WEBAPI_DEBUG
	bool bIsTypeResolved = true;
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		bIsTypeResolved = Type.TypeInfo->GetModel() != nullptr;
	}
	
	if(!bIsTypeResolved)
	{
		DebugLabel = TEXT(" (Unresolved)");
	}
#endif
	
	CachedLabel = FText::Format(LOCTEXT("ParameterLabel", "{0}: {1}"),
		FText::FromString(Name.ToString()),
		FText::FromString(Type.GetDisplayName() + (IsArray() ? TEXT("[]") : TEXT("")) + DebugLabel));

	TArray<FText> TooltipArgs;
	TooltipArgs.Emplace(LOCTEXT("ParameterTooltip_Header", "Property").ToUpper());
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_MemberName", "MemberName: {0}"), FText::FromString(Parameter->Name.ToMemberName())));
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Storage", "Storage: {0}"), FText::FromString(UE::WebAPI::WebAPIParameterStorage::ToString(Parameter->Storage))));

	if(!Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Description", "Description: {0}"), FText::FromString(Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_JsonName", "JsonName: {0}"), FText::FromString(Parameter->Name.GetJsonName())));
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Type", "Type: {0}"),	FText::FromString(Type.ToString())));

	const FString DefaultValue = Parameter->GetDefaultValue();
	if(!DefaultValue.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_DefaultValue", "Default: {0}"), FText::FromString(DefaultValue)));
	}

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

#undef LOCTEXT_NAMESPACE
