// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class UWebAPIOperationParameter;
struct FSlateColor;

class UWebAPIOperation;
class UWebAPIService;
class UWebAPIOperationParameters;
class FWebAPIOperationRequestViewModel;

/**  */
class FWebAPIOperationParameterViewModel
	: public TSharedFromThis<FWebAPIOperationParameterViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIOperationParameterViewModel> Create(const TSharedRef<FWebAPIOperationRequestViewModel>& InParentViewModel, UWebAPIOperationParameter* InParameter, int32 InIndex = -1);
	virtual ~FWebAPIOperationParameterViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_OperationParameter; }

	const TSharedPtr<FWebAPIOperationRequestViewModel>& GetRequest() const { return Request; }
	FSlateColor GetPinColor() const;

	bool IsArray() const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Parameter.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIOperationParameterViewModel(FPrivateToken) {}
	FWebAPIOperationParameterViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationRequestViewModel>& InParentViewModel, UWebAPIOperationParameter* InParameter, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<FWebAPIOperationRequestViewModel> Request;
	TWeakObjectPtr<UWebAPIOperationParameter> Parameter;
	int32 Index;
};
