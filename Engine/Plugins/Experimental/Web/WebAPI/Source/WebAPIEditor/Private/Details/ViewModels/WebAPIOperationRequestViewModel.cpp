// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOperationRequestViewModel.h"

#include "WebAPIModelViewModel.h"
#include "WebAPIOperationParameterViewModel.h"
#include "WebAPIOperationViewModel.h"
#include "Algo/AllOf.h"
#include "Dom/WebAPIOperation.h"

#define LOCTEXT_NAMESPACE "WebAPIOperationRequestViewModel"

TSharedRef<FWebAPIOperationRequestViewModel> FWebAPIOperationRequestViewModel::Create(const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex)
{
	TSharedRef<FWebAPIOperationRequestViewModel> ViewModel = MakeShared<FWebAPIOperationRequestViewModel>(FPrivateToken{}, InParentViewModel, InOperation, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIOperationRequestViewModel::GetParent()
{
	return Operation;
}

bool FWebAPIOperationRequestViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Parameters);
	if(Body.IsValid())
	{
		OutChildren.Add(Body);
	}
	return !Parameters.IsEmpty();
}

bool FWebAPIOperationRequestViewModel::IsValid() const
{
	return Algo::AllOf(Parameters, [](const TSharedPtr<FWebAPIOperationParameterViewModel>& InParameter)
	{
		return InParameter.IsValid();
	});
}

FWebAPIOperationRequestViewModel::FWebAPIOperationRequestViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex)
	: Operation(InParentViewModel)
	, OperationObject(InOperation)
	, Index(InIndex)
{
}

void FWebAPIOperationRequestViewModel::Initialize()
{
	Parameters.Empty(Parameters.Num());

	for(const TObjectPtr<UWebAPIOperationParameter>& Input : OperationObject->Request->Parameters)
	{
		TSharedRef<FWebAPIOperationParameterViewModel> ParameterViewModel = FWebAPIOperationParameterViewModel::Create(AsShared(), Input);
		Parameters.Add(MoveTemp(ParameterViewModel));
	}

	CachedLabel = FText::Format(LOCTEXT("RequestLabel", "{0}"), FText::FromString(TEXT("Request")));

	TArray<FText> TooltipArgs;

	const TObjectPtr<UWebAPIOperationRequest> Request = OperationObject->Request;

#if WITH_WEBAPI_DEBUG
	TooltipArgs.Emplace(FText::Format(LOCTEXT("RequestTooltip_FullName", "FullName: {0}"), FText::FromString(Request->Name.ToString())));
#endif
	
	if(!Request->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("RequestTooltip_Description", "Description: {0}"), FText::FromString(Request->Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("RequestTooltip_JsonName", "JsonName: {0}"), FText::FromString(Request->Name.GetJsonName())));
	
	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

#undef LOCTEXT_NAMESPACE
