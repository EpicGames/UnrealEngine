// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class FWebAPIModelViewModel;

class UWebAPIOperation;
class UWebAPIService;
class UWebAPIOperationParameters;
class FWebAPIOperationViewModel;
class FWebAPIOperationParameterViewModel;

/** Encapsulates a single Operation Request, containing one or more parameters. */
class FWebAPIOperationRequestViewModel
	: public TSharedFromThis<FWebAPIOperationRequestViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIOperationRequestViewModel> Create(const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex = -1);
	virtual ~FWebAPIOperationRequestViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_OperationRequest; }
	const TArray<TSharedPtr<FWebAPIOperationParameterViewModel>>& GetParameters() const { return Parameters; }
	bool HasBody() const { return Body.IsValid(); }
	const TSharedPtr<FWebAPIModelViewModel>& GetBody() const { return Body; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override;

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIOperationRequestViewModel(FPrivateToken) {}
	FWebAPIOperationRequestViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<FWebAPIOperationViewModel> Operation;
	TWeakObjectPtr<UWebAPIOperation> OperationObject;
	TArray<TSharedPtr<FWebAPIOperationParameterViewModel>> Parameters;
	TSharedPtr<FWebAPIModelViewModel> Body; // Re-use Model VM
	int32 Index;
};
