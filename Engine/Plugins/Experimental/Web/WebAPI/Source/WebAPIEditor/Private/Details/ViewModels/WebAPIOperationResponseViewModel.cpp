// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOperationResponseViewModel.h"

#include "WebAPIOperationViewModel.h"
#include "Dom/WebAPIOperation.h"

#define LOCTEXT_NAMESPACE "WebAPIOperationResponseViewModel"

TSharedRef<FWebAPIOperationResponseViewModel> FWebAPIOperationResponseViewModel::Create(const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperationResponse* InResponse, int32 InIndex)
{
	TSharedRef<FWebAPIOperationResponseViewModel> ViewModel = MakeShared<FWebAPIOperationResponseViewModel>(FPrivateToken{}, InParentViewModel, InResponse, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIOperationResponseViewModel::GetParent()
{
	return Operation;
}

bool FWebAPIOperationResponseViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	return false;
}

const FText& FWebAPIOperationResponseViewModel::GetDescription() const
{
	return CachedDescription;
}

FWebAPIOperationResponseViewModel::FWebAPIOperationResponseViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperationResponse* InResponse, int32 InIndex)
	: Operation(InParentViewModel)
	, Response(InResponse)
	, Index(InIndex)
{
}

void FWebAPIOperationResponseViewModel::Initialize()
{
	const uint32 ResponseCode = Response->Code;
	
	const FWebAPITypeNameVariant Name = Response->Name;
	const FWebAPITypeNameVariant Type = Response->Type;
	
	if(ResponseCode > 0)
	{
		CachedLabel = FText::Format(LOCTEXT("ResponseLabel", "Response Code: {0}"),
			ResponseCode);	
	}
	else
	{
		CachedLabel = LOCTEXT("ResponseLabel_Default", "Response Code: Default");
	}

	// Check if response is empty, but has description
	if(Response->Properties.IsEmpty() && !Response->Description.IsEmpty())
	{
		CachedDescription = FText::FromString(Response->Description);
	}

	TArray<FText> TooltipArgs;

#if WITH_WEBAPI_DEBUG
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ResponseTooltip_FullName", "FullName: {0}"), FText::FromString(Response->Name.ToString())));
#endif
	
	if(!Response->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("ResponseTooltip_Description", "Description: {0}"), FText::FromString(Response->Description)));
	}

	TooltipArgs.Emplace(FText::Format(LOCTEXT("ResponseTooltip_JsonName", "JsonName: {0}"), FText::FromString(Response->Name.GetJsonName())));
	
	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

#undef LOCTEXT_NAMESPACE
