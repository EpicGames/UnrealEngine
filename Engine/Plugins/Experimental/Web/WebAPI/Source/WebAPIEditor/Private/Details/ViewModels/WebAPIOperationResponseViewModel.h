// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class UWebAPIOperation;
class UWebAPIService;
class UWebAPIOperationResponse;
class FWebAPIOperationViewModel;

class FWebAPIOperationResponseViewModel
	: public TSharedFromThis<FWebAPIOperationResponseViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIOperationResponseViewModel> Create(const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperationResponse* InResponse, int32 InIndex = -1);
	virtual ~FWebAPIOperationResponseViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_OperationResponse; }

	const FText& GetDescription() const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Response.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIOperationResponseViewModel(FPrivateToken) {}
	FWebAPIOperationResponseViewModel(FPrivateToken, const TSharedRef<FWebAPIOperationViewModel>& InParentViewModel, UWebAPIOperationResponse* InResponse, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<FWebAPIOperationViewModel> Operation;
	TWeakObjectPtr<UWebAPIOperationResponse> Response;
	int32 Index;
	FText CachedDescription;
};
