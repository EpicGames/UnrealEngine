// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOperationViewModel.h"

#include "ScopedTransaction.h"
#include "WebAPIOperationRequestViewModel.h"
#include "WebAPIOperationResponseViewModel.h"
#include "WebAPIServiceViewModel.h"
#include "Dom/WebAPIOperation.h"

#define LOCTEXT_NAMESPACE "WebAPIOperationViewModel"

TSharedRef<FWebAPIOperationViewModel> FWebAPIOperationViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex)
{
	TSharedRef<FWebAPIOperationViewModel> ViewModel = MakeShared<FWebAPIOperationViewModel>(FPrivateToken{}, InParentViewModel, InOperation, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIOperationViewModel::GetParent()
{
	return Service;
}

bool FWebAPIOperationViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Requests);
	OutChildren.Append(Responses);
	return !Requests.IsEmpty() || !Responses.IsEmpty();
}

bool FWebAPIOperationViewModel::HasCodeText() const
{
	return IsValid() && !Operation->GeneratedCodeText.IsEmpty();
}

FText FWebAPIOperationViewModel::GetCodeText() const
{
	if(!IsValid())
	{
		return IWebAPIViewModel::GetCodeText();
	}

	return FText::FromString(Operation->GeneratedCodeText);
}

bool FWebAPIOperationViewModel::GetShouldGenerate(bool bInherit) const
{
	if(Operation.IsValid())
	{
		bool bParentShouldGenerate = true;
		if(const TSharedPtr<FWebAPIServiceViewModel> ParentService = StaticCastSharedPtr<FWebAPIServiceViewModel>(Service))
		{
			bParentShouldGenerate = ParentService->GetShouldGenerate();
		}
		
		return bInherit
			? Operation->bGenerate && bParentShouldGenerate
			: Operation->bGenerate;
	}

	return false;
}

void FWebAPIOperationViewModel::SetShouldGenerate(bool bInShouldGenerate) const
{
	if(Operation.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ModifyOperationGenerate", "Modify operation generate flag"));
		Operation->Modify();
		Operation->bGenerate = bInShouldGenerate;
	}
}

TSharedPtr<FWebAPIServiceViewModel> FWebAPIOperationViewModel::GetService() const
{
	return StaticCastSharedPtr<FWebAPIServiceViewModel>(Service);
}

FWebAPIOperationViewModel::FWebAPIOperationViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex)
	: Service(InParentViewModel)
	, Operation(InOperation)
	, Index(InIndex)
{
}

void FWebAPIOperationViewModel::Initialize()
{
	Requests.Empty(Requests.Num());
	Responses.Empty(Responses.Num());

	// As far as I know, there's only one request model, but allow for multiple
	TSharedRef<FWebAPIOperationRequestViewModel> RequestViewModel = FWebAPIOperationRequestViewModel::Create(AsShared(), Operation.Get());
	Requests.Add(MoveTemp(RequestViewModel));

	for(const TObjectPtr<UWebAPIOperationResponse>& Response : Operation->Responses)
	{
		TSharedRef<FWebAPIOperationResponseViewModel> ResponseViewModel = FWebAPIOperationResponseViewModel::Create(AsShared(), Response);
		Responses.Add(MoveTemp(ResponseViewModel));
	}

	Requests.Sort([](const TSharedPtr<FWebAPIOperationRequestViewModel>& Lhs, const TSharedPtr<FWebAPIOperationRequestViewModel>& Rhs)
	{
		return Lhs->GetLabel().ToString() < Rhs->GetLabel().ToString();
	});

	Responses.Sort([](const TSharedPtr<FWebAPIOperationResponseViewModel>& Lhs, const TSharedPtr<FWebAPIOperationResponseViewModel>& Rhs)
	{
		return Lhs->GetLabel().ToString() < Rhs->GetLabel().ToString();
	});

	const FString Name = Operation->Name.GetDisplayName();
	CachedLabel = FText::Format(LOCTEXT("OperationLabel", "{0}"),
		FText::FromString(Name));

	FString VerbStr = Operation->Verb;
	VerbStr.ToUpperInline();
	VerbLabel = FText::FromString(VerbStr);

	// Add/display path, include tokens ie. "some/path/{id}"
	const FString PathStr = Operation->Path;
	PathLabel = FText::FromString(PathStr);

	// RichPathLabel
	{
		const FTextFormat TextFormat(PathLabel);
	
		TArray<FString> PathFormatArgNames;
		TextFormat.GetFormatArgumentNames(PathFormatArgNames);

		FFormatNamedArguments PathFormatArgs;
		for(const FString& ArgName : PathFormatArgNames)
		{
			PathFormatArgs.Add(ArgName, FText::FromString(FString::Printf(TEXT("<RichTextBlock.BoldHighlight>{%s}</>"), *ArgName)));			
		}

		RichPathLabel = FText::Format(TextFormat, PathFormatArgs);
	}

	TArray<FText> TooltipArgs;

#if WITH_WEBAPI_DEBUG
	TooltipArgs.Emplace(FText::Format(LOCTEXT("OperationTooltip_FullName", "FullName: {0}"), FText::FromString(Operation->Name.ToString())));
#endif
	
	if(!Operation->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("OperationTooltip_Description", "Description: {0}"), FText::FromString(Operation->Description)));
	}

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

#undef LOCTEXT_NAMESPACE
