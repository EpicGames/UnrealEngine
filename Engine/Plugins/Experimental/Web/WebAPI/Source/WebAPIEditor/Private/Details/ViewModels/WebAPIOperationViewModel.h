// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class FWebAPIOperationRequestViewModel;
class FWebAPIOperationResponseViewModel;
class UWebAPIOperation;
class UWebAPIService;

class FWebAPIOperationViewModel
	: public TSharedFromThis<FWebAPIOperationViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIOperationViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex = -1);
	virtual ~FWebAPIOperationViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Operation; }
	virtual bool HasCodeText() const override;
	virtual FText GetCodeText() const override;

	bool GetShouldGenerate(bool bInherit = true) const;
	void SetShouldGenerate(bool bInShouldGenerate) const;
	TSharedPtr<FWebAPIServiceViewModel> GetService() const;
	const FText& GetVerb() const { return VerbLabel; }
	const FText& GetPath() const { return PathLabel; }
	const FText& GetRichPath() const { return RichPathLabel; }
	const TArray<TSharedPtr<FWebAPIOperationRequestViewModel>>& GetRequests() const { return Requests; }
	const TArray<TSharedPtr<FWebAPIOperationResponseViewModel>>& GetResponses() const { return Responses; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Operation.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIOperationViewModel(FPrivateToken) {}
	FWebAPIOperationViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIOperation* InOperation, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<IWebAPIViewModel> Service;
	TWeakObjectPtr<UWebAPIOperation> Operation;
	TArray<TSharedPtr<FWebAPIOperationRequestViewModel>> Requests;
	TArray<TSharedPtr<FWebAPIOperationResponseViewModel>> Responses;
	int32 Index;
	FText VerbLabel;
	FText PathLabel;
	FText RichPathLabel;
};
