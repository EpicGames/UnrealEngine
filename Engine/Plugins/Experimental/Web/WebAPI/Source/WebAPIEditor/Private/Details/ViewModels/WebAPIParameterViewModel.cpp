// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIParameterViewModel.h"

#include "Details/ViewModels/WebAPIOperationViewModel.h"
#include "WebAPIEnumViewModel.h"
#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "WebAPIModelViewModel.h"
#include "Dom/WebAPIParameter.h"
#include "WebAPIViewModel.inl"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPIType.h"

#define LOCTEXT_NAMESPACE "WebAPIParameterViewModel"

TSharedRef<FWebAPIParameterViewModel> FWebAPIParameterViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIParameter* InParameter, int32 InIndex)
{
	TSharedRef<FWebAPIParameterViewModel> ViewModel = MakeShared<FWebAPIParameterViewModel>(FPrivateToken{}, InParentViewModel, InParameter, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

TSharedPtr<IWebAPIViewModel> FWebAPIParameterViewModel::GetParent()
{
	return Schema;
}

bool FWebAPIParameterViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	if(Property.IsValid())
	{
		OutChildren.Add(Property);
	}
	
	if(NestedModel.IsValid())
	{
		NestedModel->GetChildren(OutChildren);
	}
	
	return Property.IsValid() || NestedModel.IsValid();
}

bool FWebAPIParameterViewModel::HasCodeText() const
{
	return IsValid() && !Parameter->GeneratedCodeText.IsEmpty();
}

FText FWebAPIParameterViewModel::GetCodeText() const
{
	if(!IsValid())
	{
		return IWebAPIViewModel::GetCodeText();
	}

	return FText::FromString(Parameter->GeneratedCodeText);
}

bool FWebAPIParameterViewModel::GetShouldGenerate() const
{
	return Parameter.IsValid() && Parameter->bGenerate;
}

bool FWebAPIParameterViewModel::IsArray() const
{
	return Parameter.IsValid() && Parameter->bIsArray;
}

FWebAPIParameterViewModel::FWebAPIParameterViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIParameter* InParameter, int32 InIndex)
	: Schema(InParentViewModel)
	, Parameter(InParameter)
	, Index(InIndex)
{
}

void FWebAPIParameterViewModel::Initialize()
{
	const FString Name = Parameter->Name.ToString();
	const FString Description = Parameter->Description;
	const FWebAPITypeNameVariant Type = Parameter->Type;
	
	if(Parameter->Property)
	{
		Property = FWebAPIPropertyViewModel::Create(AsShared(), Parameter->Property);	
	}

	if(Type.HasTypeInfo() && Type.TypeInfo->bIsNested)
	{
		NestedModel = UE::WebAPI::Details::CreateViewModel<FWebAPIParameterViewModel, IWebAPIViewModel>(SharedThis(this), Type.TypeInfo->GetModel());
	}

	CachedLabel = FText::FromString(Parameter->Name.GetDisplayName());

	TArray<FText> TooltipArgs;
	TooltipArgs.Emplace(LOCTEXT("ParameterTooltip_Header", "Parameter").ToUpper());
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Storage", "Storage: {0}"), FText::FromString(UE::WebAPI::WebAPIParameterStorage::ToString(Parameter->Storage))));

#if WITH_WEBAPI_DEBUG
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_FullName", "FullName: {0}"), FText::FromString(Parameter->Name.ToString())));
#endif
	
	if(!Parameter->Description.IsEmpty())
	{
		TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Description", "Description: {0}"), FText::FromString(Parameter->Description)));
	}
	
	TooltipArgs.Emplace(FText::Format(LOCTEXT("ParameterTooltip_Type", "Type: {0}"), FText::FromString(Parameter->Type.ToString())));

	CachedTooltip = FText::Join(FText::FromString(TEXT("\n")), TooltipArgs);
}

#undef LOCTEXT_NAMESPACE
