// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class UWebAPIParameter;

class UWebAPIOperation;
class UWebAPIService;
class FWebAPISchemaViewModel;
class FWebAPIPropertyViewModel;

/**  */
class FWebAPIParameterViewModel
	: public TSharedFromThis<FWebAPIParameterViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIParameterViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIParameter* InParameter, int32 InIndex = -1);
	virtual ~FWebAPIParameterViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Parameter; }
	virtual bool HasCodeText() const override;
	virtual FText GetCodeText() const override;

	bool GetShouldGenerate() const;
	const TSharedPtr<FWebAPIPropertyViewModel>& GetProperty() const { return Property; }

	bool IsArray() const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Parameter.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIParameterViewModel(FPrivateToken) {}
	FWebAPIParameterViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIParameter* InParameter, int32 InIndex = -1);

private:
	void Initialize();

private:
	TSharedPtr<IWebAPIViewModel> Schema;
	TWeakObjectPtr<UWebAPIParameter> Parameter;
	TSharedPtr<FWebAPIPropertyViewModel> Property;

	/** Optional nested model to display, if this properties type had to be uniquely generated and has two or more members. */
	TSharedPtr<IWebAPIViewModel> NestedModel;
	
	int32 Index;
};
