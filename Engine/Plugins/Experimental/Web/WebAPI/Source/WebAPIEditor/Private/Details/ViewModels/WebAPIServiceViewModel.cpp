// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIServiceViewModel.h"

#include "ScopedTransaction.h"
#include "WebAPIOperationViewModel.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIService.h"

#define LOCTEXT_NAMESPACE "WebAPIServiceViewModel"

TSharedRef<FWebAPIServiceViewModel> FWebAPIServiceViewModel::Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIService* InService, int32 InIndex)
{
	TSharedRef<FWebAPIServiceViewModel> ViewModel = MakeShared<FWebAPIServiceViewModel>(FPrivateToken{}, InParentViewModel, InService, InIndex);
	ViewModel->Initialize();

	return ViewModel;
}

bool FWebAPIServiceViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Operations);
	return !Operations.IsEmpty();
}

bool FWebAPIServiceViewModel::GetShouldGenerate() const
{
	if(Service.IsValid())
	{
		return Service->bGenerate;
	}

	return false;
}

void FWebAPIServiceViewModel::SetShouldGenerate(bool bInShouldGenerate) const
{
	if(Service.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ModifyServiceGenerate", "Modify service generate flag"));
		Service->Modify();
		Service->bGenerate = bInShouldGenerate;
	}
}

FWebAPIServiceViewModel::FWebAPIServiceViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIService* InService, int32 InIndex)
	: Service(InService)
	, Index(InIndex)
{
	check(Service.IsValid());
}

void FWebAPIServiceViewModel::Initialize()
{
	Operations.Empty(Operations.Num());
	for(TObjectPtr<UWebAPIOperation>& Operation : Service->Operations)
	{
		TSharedRef<FWebAPIOperationViewModel> OperationViewModel = FWebAPIOperationViewModel::Create(AsShared(), Operation);
		Operations.Add(MoveTemp(OperationViewModel));
	}

	Operations.Sort([](const TSharedPtr<FWebAPIOperationViewModel>& Lhs, const TSharedPtr<FWebAPIOperationViewModel>& Rhs)
	{
		return Lhs->GetLabel().ToString() < Rhs->GetLabel().ToString();
	});

	CachedLabel = FText::FromString(Service->Name.GetDisplayName());
}

#undef LOCTEXT_NAMESPACE
