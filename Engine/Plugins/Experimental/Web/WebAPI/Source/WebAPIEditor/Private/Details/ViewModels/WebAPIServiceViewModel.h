// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/ViewModels/WebAPIViewModel.h"

class UWebAPIService;
class FWebAPIOperationViewModel;

class FWebAPIServiceViewModel
	: public TSharedFromThis<FWebAPIServiceViewModel>
	, public IWebAPIViewModel	
{
public:
	static TSharedRef<FWebAPIServiceViewModel> Create(const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIService* InService, int32 InIndex = -1);
	virtual ~FWebAPIServiceViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override { return nullptr; }
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return Index; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Service; }

	bool GetShouldGenerate() const;
	void SetShouldGenerate(bool bInShouldGenerate) const;
	const TArray<TSharedPtr<FWebAPIOperationViewModel>>& GetOperations() const { return Operations; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Service.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIServiceViewModel(FPrivateToken) {}
	FWebAPIServiceViewModel(FPrivateToken, const TSharedRef<IWebAPIViewModel>& InParentViewModel, UWebAPIService* InService, int32 InIndex = -1);

private:
	void Initialize();

private:
	TWeakObjectPtr<UWebAPIService> Service;
	TArray<TSharedPtr<FWebAPIOperationViewModel>> Operations;
	int32 Index;
};
