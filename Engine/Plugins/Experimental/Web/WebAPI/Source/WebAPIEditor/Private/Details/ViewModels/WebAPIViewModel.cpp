// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIViewModel.h"

#include "Dom/WebAPIParameter.h"
#include "WebAPIDefinition.h"
#include "WebAPIViewModel.inl"
#include "WebAPIEnumViewModel.h"
#include "WebAPIModelViewModel.h"
#include "WebAPIParameterViewModel.h"
#include "WebAPIServiceViewModel.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPIType.h"

#define LOCTEXT_NAMESPACE "WebAPIViewModel"

FName IWebAPIViewModel::NAME_Definition = TEXT("Definition");
FName IWebAPIViewModel::NAME_Schema = TEXT("Schema");
FName IWebAPIViewModel::NAME_Model = TEXT("Model");
FName IWebAPIViewModel::NAME_Property = TEXT("Property");
FName IWebAPIViewModel::NAME_Enum = TEXT("Enum");
FName IWebAPIViewModel::NAME_EnumValue = TEXT("EnumValue");
FName IWebAPIViewModel::NAME_Parameter = TEXT("ParameterModel");
FName IWebAPIViewModel::NAME_Service = TEXT("Service");
FName IWebAPIViewModel::NAME_Operation = TEXT("Operation");
FName IWebAPIViewModel::NAME_OperationRequest = TEXT("OperationRequest");
FName IWebAPIViewModel::NAME_OperationParameter = TEXT("OperationParameter");
FName IWebAPIViewModel::NAME_OperationResponse = TEXT("OperationResponse");
FName IWebAPIViewModel::NAME_Code = TEXT("Code");
FName IWebAPIViewModel::NAME_MessageLog = TEXT("MessageLog");

FText IWebAPIViewModel::GetCodeText() const
{
	return FText::GetEmpty();
}

TSharedRef<FWebAPIDefinitionViewModel> FWebAPIDefinitionViewModel::Create(UWebAPIDefinition* InDefinition)
{
	TSharedRef<FWebAPIDefinitionViewModel> ViewModel = MakeShared<FWebAPIDefinitionViewModel>(FPrivateToken{}, InDefinition);
	ViewModel->Initialize();

	return ViewModel;
}

FWebAPIDefinitionViewModel::FWebAPIDefinitionViewModel(FPrivateToken, UWebAPIDefinition* InDefinition)
	: Definition(InDefinition)
{
	check(Definition.IsValid());
}

bool FWebAPIDefinitionViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Add(Schema);	
	return true;
}

void FWebAPIDefinitionViewModel::Refresh()
{
	Schema->Refresh();
}

bool FWebAPIDefinitionViewModel::IsSameDefinition(UObject* InObject) const
{
	if(InObject && IsValid())
	{
		return InObject == Definition.Get();
	}

	return false;
}

bool FWebAPIDefinitionViewModel::IsValid() const
{
	return Definition.IsValid();
}

void FWebAPIDefinitionViewModel::Initialize()
{
	Schema = FWebAPISchemaViewModel::Create(AsShared(), Definition->GetWebAPISchema());
	CachedLabel = FText::GetEmpty();
}

TSharedRef<FWebAPISchemaViewModel> FWebAPISchemaViewModel::Create(const TSharedRef<FWebAPIDefinitionViewModel>& InParentViewModel, UWebAPISchema* InSchema)
{
	TSharedRef<FWebAPISchemaViewModel> ViewModel = MakeShared<FWebAPISchemaViewModel>(FPrivateToken{}, InParentViewModel, InSchema);
	ViewModel->Initialize();

	return ViewModel;
}

void FWebAPISchemaViewModel::Refresh()
{
	Initialize();
}

FWebAPISchemaViewModel::FWebAPISchemaViewModel(FPrivateToken, const TSharedRef<FWebAPIDefinitionViewModel>& InParentViewModel, UWebAPISchema* InSchema)
	: Definition(InParentViewModel)
	, Schema(InSchema)
{
	check(Schema.IsValid());
}

void FWebAPISchemaViewModel::Initialize()
{
	Services.Empty(Services.Num());
	for(TTuple<FString, TObjectPtr<UWebAPIService>>& Service : Schema->Services)
	{
		TSharedRef<FWebAPIServiceViewModel> ServiceViewModel = FWebAPIServiceViewModel::Create(AsShared(), Service.Value);
		Services.Add(MoveTemp(ServiceViewModel));
	}

	TArray<TObjectPtr<UWebAPIModelBase>> SourceModels = Schema->Models;
	Algo::SortBy(SourceModels, [](const TObjectPtr<UWebAPIModelBase>& InModel)
	{
		return InModel->GetSortKey();		
	});

	Models.Empty(Models.Num());
	for(const TObjectPtr<UWebAPIModelBase>& Model : SourceModels)
	{
		if (UWebAPIEnum* SrcEnum = Cast<UWebAPIEnum>(Model))
		{
			if(SrcEnum->Name.HasTypeInfo() && SrcEnum->Name.TypeInfo->bIsNested)
			{
				continue;
			}
			TSharedRef<FWebAPIEnumViewModel> EnumViewModel = FWebAPIEnumViewModel::Create(AsShared(), SrcEnum);
			Models.Add(MoveTemp(EnumViewModel));
		}
		else if (UWebAPIParameter* SrcParameter = Cast<UWebAPIParameter>(Model))
		{
			if(SrcParameter->Name.HasTypeInfo() && SrcParameter->Name.TypeInfo->bIsNested)
			{
				continue;	
			}			
			TSharedRef<FWebAPIParameterViewModel> ParameterViewModel = FWebAPIParameterViewModel::Create(AsShared(), SrcParameter);
			Models.Add(MoveTemp(ParameterViewModel));
		}
		else if (UWebAPIModel* SrcModel = Cast<UWebAPIModel>(Model))
		{
			if(SrcModel->Name.HasTypeInfo() && SrcModel->Name.TypeInfo->bIsNested)
			{
				continue;
			}
			TSharedRef<FWebAPIModelViewModel> ModelViewModel = FWebAPIModelViewModel::Create(AsShared(), SrcModel);
			Models.Add(MoveTemp(ModelViewModel));
		}
	}

	CachedLabel = FText::GetEmpty();
}

TSharedPtr<IWebAPIViewModel> FWebAPISchemaViewModel::GetParent()
{
	return Definition;
}

bool FWebAPISchemaViewModel::GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren)
{
	OutChildren.Append(Services);
	OutChildren.Append(Models);
	return !Services.IsEmpty() && !Models.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
