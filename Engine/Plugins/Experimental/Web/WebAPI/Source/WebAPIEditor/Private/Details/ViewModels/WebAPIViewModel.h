// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

class UWebAPIDefinition;
class UWebAPISchema;

class FWebAPIDefinitionViewModel;
class FWebAPIModelViewModel;
class FWebAPISchemaViewModel;
class FWebAPIServiceViewModel;
class IWebAPISchemaObjectInterface;
class IWebAPIViewModel;

namespace UE
{
	namespace WebAPI
	{
		namespace Details
		{
			template <class ParentViewModelType = IWebAPIViewModel, class ViewModelType = IWebAPIViewModel>
			TSharedPtr<ViewModelType> CreateViewModel(const TSharedRef<ParentViewModelType>& InParentViewModel, UObject* InModel);
		}
	}
}

/** Common interface for View Models. */
class IWebAPIViewModel
{
protected:
	~IWebAPIViewModel() = default;

public:
	/** Get Parent ViewModel, if any. */
	virtual TSharedPtr<class IWebAPIViewModel> GetParent() = 0;

	/** Get Parent ViewModel of ModelType, if any. */
	template <typename ModelType>
	TSharedPtr<ModelType> GetParent();

	/** Get Label Text. */
	virtual FText GetLabel() { return CachedLabel; }

	/** Get Tooltip Text. */
	virtual FText GetTooltip() { return CachedTooltip; }
	
	/** See STreeView::OnGetChildren, return false if no children. */
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) = 0;

	/** Get the index of this element, if applicable. */
	virtual int32 GetIndex() const = 0;

	/** Get the ViewModel type name. */
	virtual const FName& GetViewModelTypeName() = 0;

	/** Whether this ViewModel should be initially expanded in a treeview. */
	virtual bool ShouldExpandByDefault() { return true; }

	/** Rebuild the ViewModel. */
	virtual void Refresh() { };

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const = 0;

	/** Check if this viewmodel context contains any code to display. */
	virtual bool HasCodeText() const { return false; }

	/** Get code text, if present. */
	virtual FText GetCodeText() const;

public:
	static FName NAME_Definition;
	static FName NAME_Schema;
	static FName NAME_Model;	
	static FName NAME_Property;
	static FName NAME_Enum;
	static FName NAME_EnumValue;
	static FName NAME_Parameter;
	static FName NAME_Service;
	static FName NAME_Operation;
	static FName NAME_OperationRequest;
	static FName NAME_OperationParameter;
	static FName NAME_OperationResponse;
	static FName NAME_Code;
	static FName NAME_MessageLog;

protected:
	FText CachedLabel;
	FText CachedTooltip;
};

template <class ModelType>
TSharedPtr<ModelType> IWebAPIViewModel::GetParent()
{
	TSharedPtr<IWebAPIViewModel> Parent = GetParent();
	while(Parent.IsValid())
	{
		if(const TSharedPtr<ModelType> FoundParentOfType = StaticCastSharedPtr<ModelType>(Parent))
		{
			return FoundParentOfType;			
		}

		// Step up the tree
		Parent = Parent->GetParent();
	}

	return nullptr;
}

class FWebAPIDefinitionViewModel
	: public TSharedFromThis<FWebAPIDefinitionViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPIDefinitionViewModel> Create(UWebAPIDefinition* InDefinition);
	virtual ~FWebAPIDefinitionViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override { return nullptr; }
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return 0; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Definition; }

	const TWeakObjectPtr<UWebAPIDefinition>& GetDefinition() const	{ return Definition; }
	TSharedPtr<FWebAPISchemaViewModel> GetSchema() const { return Schema; }

	/** Rebuild the ViewModel. */
	virtual void Refresh() override;;

	bool IsSameDefinition(UObject* InObject) const;

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override;

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPIDefinitionViewModel(FPrivateToken) {}
	FWebAPIDefinitionViewModel(FPrivateToken, UWebAPIDefinition* InDefinition);

private:
	void Initialize();

private:
	TWeakObjectPtr<UWebAPIDefinition> Definition;
	TSharedPtr<FWebAPISchemaViewModel> Schema;
};

class FWebAPISchemaViewModel
	: public TSharedFromThis<FWebAPISchemaViewModel>
	, public IWebAPIViewModel
{
public:
	static TSharedRef<FWebAPISchemaViewModel> Create(const TSharedRef<FWebAPIDefinitionViewModel>& InParentViewModel, UWebAPISchema* InSchema);
	virtual ~FWebAPISchemaViewModel() = default;

	virtual TSharedPtr<IWebAPIViewModel> GetParent() override;
	virtual bool GetChildren(TArray<TSharedPtr<IWebAPIViewModel>>& OutChildren) override;
	virtual int32 GetIndex() const override { return 0; }
	virtual const FName& GetViewModelTypeName() override { return IWebAPIViewModel::NAME_Schema; }
	
	const TArray<TSharedPtr<FWebAPIServiceViewModel>>& GetServices() const { return Services; }
	const TArray<TSharedPtr<IWebAPIViewModel>>& GetModels() const { return Models; }

	/** Rebuild the ViewModel. */
	virtual void Refresh() override;

	/** Whether this ViewModel should be initially expanded. */
	virtual bool ShouldExpandByDefault() override { return true; }

	/** Checks validity of this ViewModel */
	virtual bool IsValid() const override { return Schema.IsValid(); }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FWebAPISchemaViewModel(FPrivateToken) {}
	FWebAPISchemaViewModel(FPrivateToken, const TSharedRef<FWebAPIDefinitionViewModel>& InParentViewModel, UWebAPISchema* InSchema);

private:
	void Initialize();

private:
	TSharedPtr<FWebAPIDefinitionViewModel> Definition;
	TWeakObjectPtr<UWebAPISchema> Schema;
	TArray<TSharedPtr<FWebAPIServiceViewModel>> Services;
	TArray<TSharedPtr<IWebAPIViewModel>> Models;
};
