// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/WebAPIService.h"
#include "WebAPIEditorLog.h"
#include "WebAPIEnumViewModel.h"
#include "WebAPIModelViewModel.h"
#include "WebAPIOperationViewModel.h"
#include "WebAPIParameterViewModel.h"
#include "WebAPIServiceViewModel.h"
#include "WebAPIViewModel.h"

namespace UE::WebAPI::Details
{
	template <class ParentViewModelType, class ViewModelType>
	TSharedPtr<ViewModelType> CreateViewModel(const TSharedRef<ParentViewModelType>& InParentViewModel, UObject* InModel)
	{
		if(!InModel)
		{
			return nullptr;
		}

		const UClass* ModelClass = InModel->GetClass();
		if(ModelClass->IsChildOf<UWebAPIModelBase>())
		{
			if(ModelClass == UWebAPIEnum::StaticClass())
			{
				return FWebAPIEnumViewModel::Create(InParentViewModel, Cast<UWebAPIEnum>(InModel));
			}
			else if(ModelClass == UWebAPIEnumValue::StaticClass())
			{
				return FWebAPIEnumValueViewModel::Create(InParentViewModel, Cast<UWebAPIEnumValue>(InModel));
			}
			else if(ModelClass == UWebAPIModel::StaticClass())
			{
				return FWebAPIModelViewModel::Create(InParentViewModel, Cast<UWebAPIModel>(InModel));
			}
			else if(ModelClass == UWebAPIProperty::StaticClass())
			{
				return FWebAPIPropertyViewModel::Create(InParentViewModel, Cast<UWebAPIProperty>(InModel));
			}
			else if(ModelClass == UWebAPIService::StaticClass())
			{
				return FWebAPIServiceViewModel::Create(InParentViewModel, Cast<UWebAPIService>(InModel));
			}
			else if(ModelClass == UWebAPIParameter::StaticClass())
			{
				return FWebAPIParameterViewModel::Create(InParentViewModel, Cast<UWebAPIParameter>(InModel));
			}
			else
			{
				checkNoEntry();
				return nullptr;
			}
		}
		else if(ModelClass->IsChildOf<UWebAPIOperation>())
		{
			if(ModelClass == UWebAPIOperation::StaticClass())
			{
				return FWebAPIOperationViewModel::Create(InParentViewModel, Cast<UWebAPIOperation>(InModel));
			}
			else
			{
				checkNoEntry();
				return nullptr;
			}
		}
		else
		{
			UE_LOGF(LogWebAPIEditor, Error, "Unsupported Type");
			return nullptr;
		}
	}
}
