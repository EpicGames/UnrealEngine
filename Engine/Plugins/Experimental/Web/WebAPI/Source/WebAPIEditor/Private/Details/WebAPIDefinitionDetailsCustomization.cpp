// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDefinitionDetailsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "Editor.h"
#include "WebAPIDefinition.h"
#include "Subsystems/ImportSubsystem.h"
#include "ViewModels/WebAPIViewModel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWebAPITreeView.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "WebAPIDefinitionAssetDetailsCustomization"

FWebAPIDefinitionDetailsCustomization::FWebAPIDefinitionDetailsCustomization()
{
}

FWebAPIDefinitionDetailsCustomization::~FWebAPIDefinitionDetailsCustomization()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
}

TSharedRef<IDetailCustomization> FWebAPIDefinitionDetailsCustomization::MakeInstance()
{
	TSharedRef<FWebAPIDefinitionDetailsCustomization> Result = MakeShared<FWebAPIDefinitionDetailsCustomization>();
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddSP(Result, &FWebAPIDefinitionDetailsCustomization::OnAssetImported);
	return Result;
}

void FWebAPIDefinitionDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Register to be notified when an object is reimported.
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddRaw(this, &FWebAPIDefinitionDetailsCustomization::OnDefinitionReimported);
	
	static const FName DefaultCategoryName = UWebAPIDefinition::StaticClass()->GetFName();
	
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;	
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// Store the customized object
	TWeakObjectPtr<UObject>* FoundWebAPIDefinition = CustomizedObjects.FindByPredicate([](const TWeakObjectPtr<UObject>& Object) { return Object->IsA<UWebAPIDefinition>(); } );
	UWebAPIDefinition* WebAPIDefinition = FoundWebAPIDefinition ? Cast<UWebAPIDefinition>((*FoundWebAPIDefinition).Get()) : nullptr;
	RootViewModel = FWebAPIDefinitionViewModel::Create(WebAPIDefinition);
	
	IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory(DefaultCategoryName);
	//DefaultCategory.InitiallyCollapsed(true);
	DefaultCategory.SetCategoryVisibility(false);

	ServicesTreeView = SNew(SWebAPITreeView, LOCTEXT("ServicesLabel", "Services"))
		.OnSelectionChanged(this, &FWebAPIDefinitionDetailsCustomization::OnServiceSelectionChanged);

	ModelsTreeView = SNew(SWebAPITreeView, LOCTEXT("ModelsLabel", "Models"))
		.OnSelectionChanged(this, &FWebAPIDefinitionDetailsCustomization::OnModelSelectionChanged);

	IDetailCategoryBuilder& ApiTreeViewCategory = DetailBuilder.EditCategory("API");
	// Move to start of stack
	ApiTreeViewCategory.SetSortOrder(-100);

	IDetailCategoryBuilder& ImportSettingsCategory = DetailBuilder.EditCategory("Import Settings");
	// Move to end of stack
	ApiTreeViewCategory.SetSortOrder(100);
	
	ApiTreeViewCategory
	.AddCustomRow(FText::FromString("Schema"))
	.WholeRowContent()
	.MaxDesiredWidth(250)
	[
		SNew(SBox)
		.MinDesiredHeight(40)
		.MaxDesiredHeight(450)
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			
			+ SSplitter::Slot()
			[
				ServicesTreeView.ToSharedRef()
			]
			
			+ SSplitter::Slot()
			[
				ModelsTreeView.ToSharedRef()
			]
		]
	];

	Refresh();
}

FOnWebAPISchemaObjectSelected& FWebAPIDefinitionDetailsCustomization::OnSchemaObjectSelected()
{
	static FOnWebAPISchemaObjectSelected OnSchemaObjectSelectedDelegate;
	return OnSchemaObjectSelectedDelegate;
}

void FWebAPIDefinitionDetailsCustomization::OnAssetImported(UObject* InObject) const
{
	if(InObject && RootViewModel->IsSameDefinition(InObject))
	{
		Refresh();
	}
}

void FWebAPIDefinitionDetailsCustomization::GetChildren(FTreeItemType InObject, TArray<FTreeItemType>& OutChildren)
{
	if (!InObject.IsValid() || !InObject->IsValid())
	{
		return;
	}

	InObject->GetChildren(OutChildren);
}

void FWebAPIDefinitionDetailsCustomization::Refresh() const
{
	RootViewModel->Refresh();
	
	RefreshTreeView();
}

void FWebAPIDefinitionDetailsCustomization::RefreshTreeView() const
{
	ServicesTreeView->Refresh(RootViewModel->GetSchema()->GetServices());
	ModelsTreeView->Refresh(RootViewModel->GetSchema()->GetModels());
}

void FWebAPIDefinitionDetailsCustomization::OnServiceSelectionChanged(FTreeItemType InObject, ESelectInfo::Type InSelectInfo) const
{
	OnSchemaObjectSelected().Broadcast(RootViewModel->GetDefinition(), InObject);
}

void FWebAPIDefinitionDetailsCustomization::OnModelSelectionChanged(FTreeItemType InObject, ESelectInfo::Type InSelectInfo) const
{
	OnSchemaObjectSelected().Broadcast(RootViewModel->GetDefinition(), InObject);
}

void FWebAPIDefinitionDetailsCustomization::OnDefinitionReimported(UObject* InObject) const
{
	// Ignore if this is regarding a different object
	if (!RootViewModel->IsSameDefinition(InObject))
	{
		return;
	}

	Refresh();
}

#undef LOCTEXT_NAMESPACE
