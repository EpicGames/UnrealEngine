// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomization.h"

class FWebAPIDefinitionViewModel;
class IWebAPIViewModel;
class SWebAPITreeView;
namespace ESelectInfo { enum Type : int; }

class UWebAPIDefinition;
class IDetailLayoutBuilder;
class IWebAPISchemaObjectInterface;

/** Invoked when an object is selected in a tree view (service, model, etc.). */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWebAPISchemaObjectSelected, const TWeakObjectPtr<UWebAPIDefinition>&, const TSharedPtr<IWebAPIViewModel>&);

class FWebAPIDefinitionDetailsCustomization
    : public IDetailCustomization
{
public:
	using FTreeItemType = TSharedPtr<IWebAPIViewModel>;

	FWebAPIDefinitionDetailsCustomization();
	virtual ~FWebAPIDefinitionDetailsCustomization() override;
	
    /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/* IDetailsCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** Global delegate for WebAPI schema object selection in the UI. */
	static FOnWebAPISchemaObjectSelected& OnSchemaObjectSelected();

protected:
	void OnAssetImported(UObject* InObject) const;
	void GetChildren(FTreeItemType InObject, TArray<FTreeItemType>& OutChildren);

	/** Call when ViewModel has regenerated*/
	void Refresh() const;
	void RefreshTreeView() const;

	/** Called when the item selection changes in the services tree view. */
	void OnServiceSelectionChanged(FTreeItemType InObject, ESelectInfo::Type InSelectInfo) const;

	/** Called when the item selection changes in the models tree view. */
	void OnModelSelectionChanged(FTreeItemType InObject, ESelectInfo::Type InSelectInfo) const;

	void OnDefinitionReimported(UObject* InObject) const;

private:
	/** View Model for the current Definition, contains reference to WebAPIDefinition. */
	TSharedPtr<FWebAPIDefinitionViewModel> RootViewModel;

	/** WebAPISchema services for use by TreeView. */
	TArray<FTreeItemType> ServicesTreeViewContext;

	/** WebAPISchema services TreeView Widget. */
	TSharedPtr<SWebAPITreeView> ServicesTreeView;

	/** WebAPISchema models for use by TreeView. */
	TArray<FTreeItemType> ModelsTreeViewContext;

	/** WebAPISchema models TreeView Widget. */
	TSharedPtr<SWebAPITreeView> ModelsTreeView;	
};
