// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDefinitionTargetModuleCustomization.h"

#include "CodeGen/WebAPICodeGeneratorSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "WebAPIEditorModule.h"
#include "Widgets/SWebAPIModulePicker.h"

#define LOCTEXT_NAMESPACE "WebAPIDefinitionTargetModuleCustomization"

void FWebAPIDefinitionTargetModuleCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	NamePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FWebAPIDefinitionTargetModule, Name));
	PathPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FWebAPIDefinitionTargetModule, AbsolutePath));

	check(NamePropertyHandle.IsValid());
	check(PathPropertyHandle.IsValid());
	
	const TSharedPtr<IPropertyUtilities> PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();

	const TSharedRef<SWidget> AddModuleButton = PropertyCustomizationHelpers::MakeAddButton(
		FSimpleDelegate::CreateSP(this, &FWebAPIDefinitionTargetModuleCustomization::OnAddModule),
		TAttribute<FText>(this, &FWebAPIDefinitionTargetModuleCustomization::GetAddModuleTooltip),
		TAttribute<bool>(this, &FWebAPIDefinitionTargetModuleCustomization::CanAddModule));

	FString ModuleName;
	NamePropertyHandle->GetValue(ModuleName);

	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	.MaxDesiredWidth(400.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(SWebAPIModulePicker)
			.ModuleName(ModuleName)
			.OnModuleChanged_Lambda([NamePropertyHandle = NamePropertyHandle, PathPropertyHandle = PathPropertyHandle](const FString& InModuleName, const FString& InModulePath)
			{		
				NamePropertyHandle->SetValue(InModuleName);
				PathPropertyHandle->SetValue(InModulePath);
			})
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.0f, 1.0f, 0.0f, 1.0f)
		[
			AddModuleButton
		]
	];
}

void FWebAPIDefinitionTargetModuleCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FWebAPIDefinitionTargetModuleCustomization::OnAddModule()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FWebAPIEditorModule::PluginCreatorTabName);
}

bool FWebAPIDefinitionTargetModuleCustomization::CanAddModule() const
{
	return true;
}

FText FWebAPIDefinitionTargetModuleCustomization::GetAddModuleTooltip() const
{
	return LOCTEXT("AddModuleTooltip", "Add a new module or plugin");
}

#undef LOCTEXT_NAMESPACE
