// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

/** Property customization for FWebAPIDefinitionTargetModule */
class FWebAPIDefinitionTargetModuleCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FWebAPIDefinitionTargetModuleCustomization>();
	}
	
	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Root property handle. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Property handle for Name. */
	TSharedPtr<IPropertyHandle> NamePropertyHandle;

	/** Property handle for AbsolutePath. */
	TSharedPtr<IPropertyHandle> PathPropertyHandle;

	/** Called when the "Add Module" button is pressed. */
	void OnAddModule();

	/** Returns true if the "Add Module" button can be pressed. */
	bool CanAddModule() const;

	/** The tooltip attribute for the "Add Module" button. */
	FText GetAddModuleTooltip() const;
};
