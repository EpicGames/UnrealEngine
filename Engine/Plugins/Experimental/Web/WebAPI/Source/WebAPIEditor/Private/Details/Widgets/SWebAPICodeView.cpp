// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPICodeView.h"

#include "Details/ViewModels/WebAPIViewModel.h"
#include "SlateOptMacros.h"
#include "Details/WebAPIDefinitionDetailsCustomization.h"
#include "Details/ViewModels/WebAPICodeViewModel.h"
#include "WebAPIDefinition.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScrollBox.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

SWebAPICodeView::SWebAPICodeView()
{
	FWebAPIDefinitionDetailsCustomization::OnSchemaObjectSelected().AddRaw(this, &SWebAPICodeView::OnSchemaObjectSelected);
}

SWebAPICodeView::~SWebAPICodeView()
{
	FWebAPIDefinitionDetailsCustomization::OnSchemaObjectSelected().RemoveAll(this);
}

void SWebAPICodeView::OnSchemaObjectSelected(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<IWebAPIViewModel>& InSchemaObjectViewModel) const
{
	if(CodeView && InDefinition.IsValid() && InSchemaObjectViewModel.IsValid()
		&& CodeView->GetDefinition() == InDefinition
		&& InSchemaObjectViewModel->HasCodeText())
	{
		CodeView->SetCodeText(InSchemaObjectViewModel->GetCodeText());
	}
}

void SWebAPICodeView::Construct(const FArguments& InArgs, const TSharedRef<FWebAPICodeViewModel>& InViewModel)
{
	CodeView = InViewModel;
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		.Padding(2.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew(SEditableTextBox)
				.Text_Lambda([&]
				{
					return CodeView->GetCodeText();
				})
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
