// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FWebAPICodeViewModel;

/** Shows the generated code for the selected object. */
class WEBAPIEDITOR_API SWebAPICodeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWebAPICodeView)
	{ }
	SLATE_END_ARGS()

	SWebAPICodeView();
	virtual ~SWebAPICodeView();
	
	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPICodeViewModel>& InViewModel);

protected:
	void OnSchemaObjectSelected(const TWeakObjectPtr<class UWebAPIDefinition>& InDefinition, const TSharedPtr<class IWebAPIViewModel>& InSchemaObjectViewModel) const;
	
private:
	TSharedPtr<FWebAPICodeViewModel> CodeView;
};
