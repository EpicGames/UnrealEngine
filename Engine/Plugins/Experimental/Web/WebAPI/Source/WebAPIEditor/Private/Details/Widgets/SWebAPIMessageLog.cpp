// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPIMessageLog.h"

#include "MessageLogModule.h"
#include "WebAPIMessageLog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"

void SWebAPIMessageLog::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIMessageLog>& InMessageLog)
{
	// Create widget from MessageLog module
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	LogListingWidget = MessageLogModule.CreateLogListingWidget(InMessageLog->GetMessageLogListing().ToSharedRef());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		.Padding(2.f)
		[
			LogListingWidget.ToSharedRef()
		]
	];
}
