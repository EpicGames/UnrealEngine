// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FWebAPIMessageLog;

class SWidget;

/** Lists WebAPI log entries. */
class WEBAPIEDITOR_API SWebAPIMessageLog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWebAPIMessageLog)
	{ }
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIMessageLog>& InMessageLog);

private:
	/** Pointer to log widget */
	TSharedPtr<SWidget> LogListingWidget;

	TSharedPtr<FWebAPIMessageLog> MessageLog;
};
