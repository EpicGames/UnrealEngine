// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPIModulePicker.h"

#include "Editor.h"
#include "Framework/Views/TableViewMetadata.h"
#include "GameProjectUtils.h"
#include "ModuleDescriptor.h"
#include "WebAPIEditorSubsystem.h"
#include "Misc/App.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "WebAPIModulePicker"

/** The last selected module name. Meant to keep the same module selected after first selection. */
FString SWebAPIModulePicker::LastSelectedModuleName;

void SWebAPIModulePicker::Construct(const FArguments& InArgs)
{
	ModuleName = InArgs._ModuleName;
	OnModuleChanged = InArgs._OnModuleChanged;
	
	GEditor->GetEditorSubsystem<UWebAPIEditorSubsystem>()->OnModuleCreated().AddSP(this, &SWebAPIModulePicker::OnModuleCreated);	
	Refresh();
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(AvailableModulesCombo, SComboBox<TSharedPtr<FModuleContextInfo>>)
			.ToolTipText( LOCTEXT("ModuleComboToolTip", "Choose the target module for your generated files") )
			.OptionsSource( &AvailableModules )
			.InitiallySelectedItem( SelectedModuleInfo )
			.OnSelectionChanged( this, &SWebAPIModulePicker::SelectedModuleComboBoxSelectionChanged )
			.OnGenerateWidget( this, &SWebAPIModulePicker::MakeWidgetForSelectedModuleCombo )
			[
				SNew(STextBlock)
				.Text( this, &SWebAPIModulePicker::GetSelectedModuleComboText )
			]
		]
	];
}

void SWebAPIModulePicker::Refresh()
{
	{
		TArray<FModuleContextInfo> CurrentModules = GameProjectUtils::GetCurrentProjectModules();
		check(!CurrentModules.IsEmpty());

		const TArray<FModuleContextInfo> CurrentPluginModules = GameProjectUtils::GetCurrentProjectPluginModules();

		CurrentModules.Append(CurrentPluginModules);

		AvailableModules.Reserve(CurrentModules.Num());
		for(FModuleContextInfo& ModuleInfo : CurrentModules)
		{
			AvailableModules.Emplace(MakeShared<FModuleContextInfo>(MoveTemp(ModuleInfo)));
		}

		Algo::SortBy(AvailableModules, &FModuleContextInfo::ModuleName);
	}

	// If we've been given an initial path that maps to a valid project module, use that as our initial module and path
	for(const auto& AvailableModule : AvailableModules)
	{
		if(AvailableModule->ModuleName.Equals(ModuleName, ESearchCase::IgnoreCase))
		{
			SelectedModuleInfo = AvailableModule;
			break;
		}
	}

	// If we didn't get given a valid path override (see above), try and automatically work out the best default module
	// If we have a runtime module with the same name as our project, then use that
	// Otherwise, set out default target module as the first runtime module in the list
	if(!SelectedModuleInfo.IsValid())
	{
		const FString ProjectName = FApp::GetProjectName();

		// Find initially selected module based on simple fallback in this order..
		// Previously selected module, main project module, a  runtime module
		TSharedPtr<FModuleContextInfo> ProjectModule;
		TSharedPtr<FModuleContextInfo> RuntimeModule;

		for (const auto& AvailableModule : AvailableModules)
		{
			// Check if this module matches our last used
			if (AvailableModule->ModuleName == LastSelectedModuleName)
			{
				SelectedModuleInfo = AvailableModule;
				break;
			}

			if (AvailableModule->ModuleName == ProjectName)
			{
				ProjectModule = AvailableModule;
			}

			if (AvailableModule->ModuleType == EHostType::Runtime)
			{
				RuntimeModule = AvailableModule;
			}
		}

		if (!SelectedModuleInfo.IsValid())
		{
			if (ProjectModule.IsValid())
			{
				// use the project module we found
				SelectedModuleInfo = ProjectModule;
			}
			else if (RuntimeModule.IsValid())
			{
				// use the first runtime module we found
				SelectedModuleInfo = RuntimeModule;
			}
			else
			{
				// default to just the first module
				SelectedModuleInfo = AvailableModules[0];
			}
		}

		ModuleName = SelectedModuleInfo->ModuleSourcePath;
	}
}

FText SWebAPIModulePicker::GetSelectedModuleComboText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ModuleName"), FText::FromString(SelectedModuleInfo->ModuleName));
	Args.Add(TEXT("ModuleType"), FText::FromString(EHostType::ToString(SelectedModuleInfo->ModuleType)));
	return FText::Format(LOCTEXT("ModuleComboEntry", "{ModuleName} ({ModuleType})"), Args);
}

void SWebAPIModulePicker::SelectedModuleComboBoxSelectionChanged(TSharedPtr<FModuleContextInfo> InValue, ESelectInfo::Type InSelectInfo)
{
	SelectedModuleInfo = InValue;
	OnModuleChanged.ExecuteIfBound(SelectedModuleInfo->ModuleName, SelectedModuleInfo->ModuleSourcePath);
}

TSharedRef<SWidget> SWebAPIModulePicker::MakeWidgetForSelectedModuleCombo(TSharedPtr<FModuleContextInfo> InValue) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ModuleName"), FText::FromString(InValue->ModuleName));
	Args.Add(TEXT("ModuleType"), FText::FromString(EHostType::ToString(InValue->ModuleType)));
	return SNew(STextBlock)
		.Text(FText::Format(LOCTEXT("ModuleComboEntry", "{ModuleName} ({ModuleType})"), Args));
}

void SWebAPIModulePicker::OnModuleCreated(FString InModuleName)
{
	Refresh();
}

#undef LOCTEXT_NAMESPACE
