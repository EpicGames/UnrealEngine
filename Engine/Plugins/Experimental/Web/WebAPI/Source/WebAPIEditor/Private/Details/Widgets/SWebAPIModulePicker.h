// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
struct FModuleContextInfo;
template <typename OptionType> class SComboBox;

class IClassViewerFilter;
class SClassViewer;
class SEditableTextBox;
class SWizard;
struct FParentClassItem;

DECLARE_DELEGATE_TwoParams(FOnModuleChanged, const FString& /** Name */, const FString& /** Path */);

/** A widget to select a target module or plugin for generated files. */
class SWebAPIModulePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWebAPIModulePicker)
	{}
		/** The module initially provided. */
		SLATE_ARGUMENT(FString, ModuleName)

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT(FOnModuleChanged, OnModuleChanged)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs. */
	void Construct(const FArguments& InArgs);

	/** Refreshes the list of available modules, accounting for any added or removed. */
	void Refresh();

private:
	/** Get the combo box text for the currently selected module */
	FText GetSelectedModuleComboText() const;

	/** Called when the currently selected module is changed */
	void SelectedModuleComboBoxSelectionChanged(TSharedPtr<FModuleContextInfo> InValue, ESelectInfo::Type InSelectInfo);

	/** Create the widget to use as the combo box entry for the given module info */
	TSharedRef<SWidget> MakeWidgetForSelectedModuleCombo(TSharedPtr<FModuleContextInfo> InValue) const;

	/** When a module/plugin is newly created. */
	void OnModuleCreated(FString InModuleName);
	
private:
	/** The available modules combo box */
	TSharedPtr<SComboBox<TSharedPtr<FModuleContextInfo>>> AvailableModulesCombo;

	/** The last selected module name. Meant to keep the same module selected after first selection */
	static FString LastSelectedModuleName;

	/** The ModuleName chosen. */
	FString ModuleName;

	/** Information about the currently available modules for this project */
	TArray<TSharedPtr<FModuleContextInfo>> AvailableModules;

	/** Information about the currently selected module; used for class validation */
	TSharedPtr<FModuleContextInfo> SelectedModuleInfo;

    FOnModuleChanged OnModuleChanged;
};
