// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaEnumValueRow.h"

#include "Details/ViewModels/WebAPIEnumViewModel.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

void SWebAPISchemaEnumValueRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIEnumValueViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	FSlateColor TypeColor = FSlateColor();
	
	SWebAPISchemaTreeTableRow::Construct(
	SWebAPISchemaTreeTableRow::FArguments()
	.Content()
	[
		SNew(SBorder)
		.ToolTipText(InViewModel->GetTooltip())
		.BorderBackgroundColor(FLinearColor(0,0,0,0))
		.Padding(4)
		[
			SNew(SBox)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[
					SNew(SBox)
					[
						SNew(SHorizontalBox)						
						+ SHorizontalBox::Slot()
						.Padding(9, 0, 0, 1)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "PlacementBrowser.Asset.Name")
							.Text(InViewModel->GetLabel())
						]
					]
				]
			]
		]
	],
	InViewModel,
	InOwnerTableView);
}
