// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIEnumValueViewModel;

class SWebAPISchemaEnumValueRow
	: public SWebAPISchemaTreeTableRow<FWebAPIEnumValueViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaEnumValueRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIEnumValueViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
