// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaModelRow.h"

#include "WebAPIEditorStyle.h"
#include "Details/ViewModels/WebAPIModelViewModel.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "WebAPISchemaModelRow"

void SWebAPISchemaModelRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIModelViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	SWebAPISchemaTreeTableRow::Construct(
		SWebAPISchemaTreeTableRow::FArguments()
		.IsEnabled(InViewModel, &FWebAPIModelViewModel::GetShouldGenerate)
		.Content()
		[
			SNew(SBox)
			[
				SNew(SHorizontalBox)

				// Model Type
				+ SHorizontalBox::Slot()
				.Padding(0)
				.AutoWidth()
				[
					SNew(SBorder)
					.IsEnabled(InViewModel, &FWebAPIModelViewModel::GetShouldGenerate)
					.BorderImage(FWebAPIEditorStyle::Get().GetBrush("WebAPI.TreeView.LabelBackground"))
					[
						SNew(SBox)
						.WidthOverride(60)
						.HeightOverride(20)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ModelType", "Model"))
							.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
							.TransformPolicy(ETextTransformPolicy::ToUpper)
						]
					]
				]
		
				+ SHorizontalBox::Slot()
				.Padding(0)
				.FillWidth(1.0f)
				[
					SNew(SBox)
					.IsEnabled(InViewModel, &FWebAPIModelViewModel::GetShouldGenerate)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(9, 0, 0, 1)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(InViewModel->GetLabel())
							.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNullWidget::NullWidget
						]
					]
				]
			]
		],
		InViewModel,
		InOwnerTableView);
}

#undef LOCTEXT_NAMESPACE
