// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

class FWebAPIModelViewModel;

class SWebAPISchemaModelRow
	: public SWebAPISchemaTreeTableRow<FWebAPIModelViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaModelRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIModelViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
