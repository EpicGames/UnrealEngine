// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaOperationParameterRow.h"

#include "Details/ViewModels/WebAPIOperationParameterViewModel.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

void SWebAPISchemaOperationParameterRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIOperationParameterViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	const FSlateBrush* ArrayIcon = FAppStyle::GetBrush(TEXT("Graph.ArrayPin.Connected"));
	const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("Graph.Pin.Connected"));
	
	const FSlateColor TypeColor = InViewModel->GetPinColor();
	
	SWebAPISchemaTreeTableRow::Construct(
	SWebAPISchemaTreeTableRow::FArguments()
	.Content()
	[
		SNew(SBorder)
		.ToolTipText(InViewModel->GetTooltip())
		.BorderBackgroundColor(FLinearColor(0,0,0,0))
		.Padding(4)
		[
			SNew(SBox)
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.Padding(0)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(20)
					.HeightOverride(20)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image_Lambda([&, ArrayIcon, Icon]
						{
							return ViewModel->IsArray()
									? ArrayIcon
									: Icon;
						})
						.ColorAndOpacity(TypeColor)
					]
				]
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[
					SNew(SBox)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(9, 0, 0, 1)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "PlacementBrowser.Asset.Name")
							.Text(InViewModel->GetLabel())
						]
					]
				]
			]
		]
	],
	InViewModel,
	InOwnerTableView);
}
