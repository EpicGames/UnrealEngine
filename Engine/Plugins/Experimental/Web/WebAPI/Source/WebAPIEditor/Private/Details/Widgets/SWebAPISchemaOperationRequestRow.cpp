// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaOperationRequestRow.h"

#include "Details/ViewModels/WebAPIOperationRequestViewModel.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

void SWebAPISchemaOperationRequestRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIOperationRequestViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	SWebAPISchemaTreeTableRow::Construct(
		SWebAPISchemaTreeTableRow::FArguments()
		.Content()
		[
			SNew(SBorder)
			.ToolTipText(InViewModel->GetTooltip())
			.BorderBackgroundColor(FLinearColor(0,0,0,0))
			.Padding(4)
			[
				SNew(SBox)
				[
					SNew(SHorizontalBox)
								
					+ SHorizontalBox::Slot()
					.Padding(0)
					.FillWidth(1.0f)
					[
						SNew(SBorder)
						.BorderBackgroundColor(FLinearColor(0,0,0,0))
						[
							SNew(SBox)
							[
								SNew(SHorizontalBox)
								
								+ SHorizontalBox::Slot()
								.Padding(9, 0, 0, 1)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(InViewModel->GetLabel())
									.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
								]
								
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNullWidget::NullWidget
								]
							]
						]
					]
				]
			]
		],
		InViewModel,
		InOwnerTableView);
}
