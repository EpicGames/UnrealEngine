// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIOperationRequestViewModel;

class FWebAPIEnumViewModel;

class SWebAPISchemaOperationRequestRow
	: public SWebAPISchemaTreeTableRow<FWebAPIOperationRequestViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaOperationRequestRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIOperationRequestViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
