// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIOperationResponseViewModel;

class FWebAPIEnumViewModel;

class SWebAPISchemaOperationResponseRow
	: public SWebAPISchemaTreeTableRow<FWebAPIOperationResponseViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaOperationResponseRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIOperationResponseViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
