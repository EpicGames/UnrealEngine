// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaOperationRow.h"

#include "WebAPIEditorStyle.h"
#include "Details/ViewModels/WebAPIOperationViewModel.h"
#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"

void SWebAPISchemaOperationRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIOperationViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	SWebAPISchemaTreeTableRow::Construct(
		SWebAPISchemaTreeTableRow::FArguments()
		.IsEnabled(InViewModel, &FWebAPIOperationViewModel::GetShouldGenerate, true)
		.Content()
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0,0,0,0)) // hide drawn border
			.Padding(4)
			[
				SNew(SBox)
				[
					SNew(SVerticalBox)

					// Main row
					+ SVerticalBox::Slot()
					.Padding(0)
					[
						SNew(SHorizontalBox)

						// Generation Toggle
						+ SHorizontalBox::Slot()
						.Padding(0)
						.AutoWidth()
						[
							SNew(SBox)
							.IsEnabled(InViewModel->GetService().Get(), &FWebAPIServiceViewModel::GetShouldGenerate)
							.WidthOverride(20)
							.HeightOverride(20)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SCheckBox)
								.OnCheckStateChanged_Lambda([InViewModel](const ECheckBoxState InCheckState) { InViewModel->SetShouldGenerate(InCheckState == ECheckBoxState::Checked); })
								.IsChecked_Lambda([InViewModel]
									{ return InViewModel->GetShouldGenerate(false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							]
						]

						// HTTP Verb
						+ SHorizontalBox::Slot()
						.Padding(0)
						.AutoWidth()
						[
							SNew(SBorder)
							.IsEnabled(InViewModel, &FWebAPIOperationViewModel::GetShouldGenerate, true)
							.BorderImage(FWebAPIEditorStyle::Get().GetBrush("WebAPI.TreeView.LabelBackground"))
							[
								SNew(SBox)
								.WidthOverride(60)
								.HeightOverride(20)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(InViewModel->GetVerb())
									.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
									.TransformPolicy(ETextTransformPolicy::ToUpper)
								]
							]
						]

						// Operation Label
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Fill)
						.Padding(0)
						[
							SNew(SBox)
							.IsEnabled(InViewModel, &FWebAPIOperationViewModel::GetShouldGenerate, true)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.Padding(9, 0, 0, 1)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(InViewModel->GetLabel())
									.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
									.TextStyle(FAppStyle::Get(), "PlacementBrowser.Asset.Name")								
								]
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNullWidget::NullWidget
								] 
							]
						]
					]

					// Path row
					+ SVerticalBox::Slot()
					.Padding(0)
					[
						SNew(SHorizontalBox)

						// Operation Path
						+ SHorizontalBox::Slot()
						// 20 = Generation Toggle checkbox width
						.Padding(20, 9, 0, 1)
						.VAlign(VAlign_Center) 
						[
							SNew(SRichTextBlock)
							.Text(InViewModel->GetRichPath())
						]
					]
				]
			]
		],
		InViewModel,
		InOwnerTableView);
}
