// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaParameterRow.h"

#include "Details/ViewModels/WebAPIParameterViewModel.h"
#include "WebAPIEditorStyle.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "WebAPISchemaParameterRow"

void SWebAPISchemaParameterRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIParameterViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	SWebAPISchemaTreeTableRow::Construct(
		SWebAPISchemaTreeTableRow::FArguments()
		.Content()
		[
			SNew(SBox)
			[
				SNew(SHorizontalBox)

				// Model Type
				+ SHorizontalBox::Slot()
				.Padding(0)
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage(FWebAPIEditorStyle::Get().GetBrush("WebAPI.TreeView.LabelBackground"))
					[
						SNew(SBox)
						.WidthOverride(60)
						.HeightOverride(20)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ModelType", "Param"))
							.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
							.TransformPolicy(ETextTransformPolicy::ToUpper)
						]
					]
				]
		
				+ SHorizontalBox::Slot()
				.Padding(0)
				.FillWidth(1.0f)
				[
					SNew(SBox)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(9, 0, 0, 1)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(InViewModel->GetLabel())
							.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNullWidget::NullWidget
						]
					]
				]
			]
		],
		InViewModel,
		InOwnerTableView);;
}

#undef LOCTEXT_NAMESPACE
