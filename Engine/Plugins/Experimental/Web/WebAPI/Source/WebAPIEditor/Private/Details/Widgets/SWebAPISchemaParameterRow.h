// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIParameterViewModel;

class SWebAPISchemaParameterRow
	: public SWebAPISchemaTreeTableRow<FWebAPIParameterViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaParameterRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIParameterViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
