// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIPropertyViewModel;

class SWebAPISchemaPropertyRow
	: public SWebAPISchemaTreeTableRow<FWebAPIPropertyViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaPropertyRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIPropertyViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
