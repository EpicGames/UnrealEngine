// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaServiceRow.h"

#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Details/Widgets/SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"

void SWebAPISchemaServiceRow::Construct(const FArguments& InArgs, const TSharedRef<FWebAPIServiceViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	SWebAPISchemaTreeTableRow::Construct(
		SWebAPISchemaTreeTableRow::FArguments()
		.Content()
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0,0,0,0)) // hide drawn border
			.Padding(4)
			[
				SNew(SBox)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0)
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(20)
						.HeightOverride(20)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SCheckBox)
							.OnCheckStateChanged_Lambda([InViewModel](const ECheckBoxState InCheckState) { InViewModel->SetShouldGenerate(InCheckState == ECheckBoxState::Checked); })
							.IsChecked_Lambda([InViewModel]
								{ return InViewModel->GetShouldGenerate() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						]
					]
								
					+ SHorizontalBox::Slot()
					.Padding(0)
					.FillWidth(1.0f)
					[
						SNew(SBox)
						.IsEnabled(InViewModel, &FWebAPIServiceViewModel::GetShouldGenerate)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(9, 0, 0, 1)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.IsEnabled(InViewModel, &FWebAPIServiceViewModel::GetShouldGenerate)
								.Text(InViewModel->GetLabel())
								.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNullWidget::NullWidget
							]
						]
					]
				]
			]
		],
		InViewModel,
		InOwnerTableView);
}
