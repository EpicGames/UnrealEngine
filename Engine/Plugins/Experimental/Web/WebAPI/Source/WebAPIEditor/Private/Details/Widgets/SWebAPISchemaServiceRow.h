// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Details/Widgets/SWebAPISchemaTreeTableRow.h"
#include "SWebAPISchemaTreeTableRow.inl"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class FWebAPIServiceViewModel;

class SWebAPISchemaServiceRow
	: public SWebAPISchemaTreeTableRow<FWebAPIServiceViewModel>
{
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaServiceRow)
	{ }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FWebAPIServiceViewModel>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);
};
