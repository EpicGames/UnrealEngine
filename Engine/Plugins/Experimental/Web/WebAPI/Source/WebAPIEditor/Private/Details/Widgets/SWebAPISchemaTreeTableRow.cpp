// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPISchemaTreeTableRow.h"

#include "Details/ViewModels/WebAPIViewModel.h"

#define LOCTEXT_NAMESPACE "WebAPISchemaTreeItemWidget"

constexpr float IndentSize = 12;

TSharedPtr<IWebAPIViewModel> GetEntryAbove(TSharedPtr<IWebAPIViewModel> InEntry)
{
	const TSharedPtr<IWebAPIViewModel> Outer = InEntry->GetParent();
	if (Outer)
	{
		TArray<TSharedPtr<IWebAPIViewModel>> Children;
		Outer->GetChildren(Children);
		for (int32 Index = 1; Index < Children.Num(); Index++)
		{
			if (Children[Index] == InEntry)
			{
				return Children[Index - 1];
			}
		}
	}
	return nullptr;
}

TSharedPtr<IWebAPIViewModel> GetEntryBelow(TSharedPtr<IWebAPIViewModel> InEntry)
{
	const TSharedPtr<IWebAPIViewModel> Outer = InEntry->GetParent();
	if (Outer)
	{
		TArray<TSharedPtr<IWebAPIViewModel>> Children;
		Outer->GetChildren(Children);
		for (int32 Index = 0; Index < Children.Num() - 1; Index++)
		{
			if (Children[Index] == InEntry)
			{
				return Children[Index + 1];
			}
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
