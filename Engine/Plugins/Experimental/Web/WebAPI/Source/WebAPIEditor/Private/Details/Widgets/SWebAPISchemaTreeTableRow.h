// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"

class STableViewBase;

class IWebAPISchemaTreeTableRow
{
};

template <typename ItemType>
class SWebAPISchemaTreeTableRow
	: public STableRow<TSharedRef<ItemType>>
	, public IWebAPISchemaTreeTableRow
{
	static_assert(TIsValidListItem<TSharedRef<ItemType>>::Value, "ItemType must be derived from IWebAPIViewModel.");
	
public:
	DECLARE_DELEGATE_OneParam(FOnColumnWidthChanged, float);
	
public:
	SLATE_BEGIN_ARGS(SWebAPISchemaTreeTableRow)
	{ }
		SLATE_DEFAULT_SLOT(typename SWebAPISchemaTreeTableRow<ItemType>::FArguments, Content)
	SLATE_END_ARGS()

public:
	/** Use this to construct the superclass; e.g. FSuperRowType::Construct( FTableRowArgs(), OwnerTableView ) */
	typedef STableRow<TSharedRef<ItemType>> FSuperRowType;
	
	void Construct(const FArguments& InArgs, const TSharedRef<ItemType>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView);

protected:
	TSharedPtr<ItemType> ViewModel;
};
