// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SWebAPISchemaTreeTableRow.h"

template <typename ItemType>
void SWebAPISchemaTreeTableRow<ItemType>::Construct(const FArguments& InArgs, const TSharedRef<ItemType>& InViewModel, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	ViewModel = InViewModel;

	typename FSuperRowType::FArguments InnerArgs;
	InnerArgs
	.Padding(0)
	.Content()
	[
		// Wrap in border to use common Tooltip getter
		SNew(SBorder)
		.BorderBackgroundColor(FLinearColor(0, 0, 0, 0))
		.ToolTipText(InViewModel->GetTooltip())
		[
			InArgs._Content.Widget
		]
	];
	
	STableRow<TSharedRef<ItemType>>::Construct(MoveTemp(InnerArgs), InOwnerTableView);
}
