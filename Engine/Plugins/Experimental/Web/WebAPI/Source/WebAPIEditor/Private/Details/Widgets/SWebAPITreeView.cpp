// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebAPITreeView.h"

#include "Algo/ForEach.h"
#include "Details/ViewModels/WebAPIEnumViewModel.h"
#include "Details/Widgets/SWebAPISchemaEnumRow.h"
#include "Details/ViewModels/WebAPIModelViewModel.h"
#include "Details/Widgets/SWebAPISchemaEnumValueRow.h"
#include "Details/ViewModels/WebAPIOperationParameterViewModel.h"
#include "Details/Widgets/SWebAPISchemaModelRow.h"
#include "Details/ViewModels/WebAPIOperationRequestViewModel.h"
#include "Details/Widgets/SWebAPISchemaOperationParameterRow.h"
#include "Details/ViewModels/WebAPIOperationResponseViewModel.h"
#include "Details/Widgets/SWebAPISchemaOperationRequestRow.h"
#include "Details/ViewModels/WebAPIOperationViewModel.h"
#include "Details/Widgets/SWebAPISchemaOperationResponseRow.h"
#include "Details/ViewModels/WebAPIParameterViewModel.h"
#include "Details/Widgets/SWebAPISchemaOperationRow.h"
#include "Details/ViewModels/WebAPIServiceViewModel.h"
#include "Details/Widgets/SWebAPISchemaParameterRow.h"
#include "Details/Widgets/SWebAPISchemaPropertyRow.h"
#include "Details/Widgets/SWebAPISchemaServiceRow.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SSearchBox.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SWebAPITreeView"

/** Filter utility class, used by search. */
class FWebAPITreeViewFilterContext : public ITextFilterExpressionContext
{
public:
	explicit FWebAPITreeViewFilterContext(const FText& InString)
		: String(InString)
	{
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return TextFilterUtils::TestBasicStringExpression(String.ToString(), InValue, InTextComparisonMode);
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}

private:
	const FText& String;
};

void SWebAPITreeView::Construct(const FArguments& InArgs, const FText& InLabel, const TArray<FTreeItemType>& InTreeViewItems)
{
	static const FName ColumnName("Name");

	TreeViewItems = InTreeViewItems;
	TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);
	
	TreeViewWidget =
			SNew(STreeView<FTreeItemType>)
			.SelectionMode(ESelectionMode::Multi)
			.TreeItemsSource(&FilteredTreeViewItems)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.OnGetChildren(this, &SWebAPITreeView::GetTreeItemChildren)
			.OnGenerateRow(this, &SWebAPITreeView::GenerateRow)
			.HeaderRow(
				SNew(SHeaderRow)
				.Visibility(EVisibility::Visible)
				+SHeaderRow::Column(ColumnName)
				.DefaultLabel(InLabel)
			);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &SWebAPITreeView::OnFilterTextChanged)
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, TreeViewWidget.ToSharedRef())
			[
				TreeViewWidget.ToSharedRef()
			]
		]
	];
}

void SWebAPITreeView::Refresh(const TArray<TSharedPtr<IWebAPIViewModel>>& InTreeViewModels)
{
	if(!TreeViewWidget.IsValid())
	{
		return;
	}

	if(!InTreeViewModels.IsEmpty())
	{
		TreeViewItems = InTreeViewModels;
	}

	TUniqueFunction<void(const FTreeItemType&)> ExpandRecursive = [&](const FTreeItemType& InParent)
	{
		if(!InParent->ShouldExpandByDefault())
		{
			return;			
		}
		
		TreeViewWidget->SetItemExpansion(InParent, true);
		
		TArray<TSharedPtr<IWebAPIViewModel>> Children;
		InParent->GetChildren(Children);
		Algo::ForEach(Children, [&ExpandRecursive](const FTreeItemType& InChild)
		{
			ExpandRecursive(InChild);
		});
	};

	Algo::ForEach(TreeViewItems, [&ExpandRecursive](const FTreeItemType& InChild)
	{
		ExpandRecursive(InChild);
	});

	if (FilterText.IsEmpty())
	{
		FilteredTreeViewItems = TreeViewItems;
	}
	else
	{
		TArray<TSharedPtr<IWebAPIViewModel>> AllTreeViewItems;

		// Flatten hierarchy and expand top-level items
		for(const FTreeItemType& TreeViewItem : TreeViewItems)
		{
			TArray<TSharedPtr<IWebAPIViewModel>> TreeViewItemChildren;
			TreeViewItem->GetChildren(AllTreeViewItems);

			// Always expand first level
			TreeViewWidget->SetItemExpansion(TreeViewItem, true);
		}

		for(const FTreeItemType& TreeViewItem : AllTreeViewItems)
		{
			if(TextFilter->TestTextFilter(FWebAPITreeViewFilterContext(TreeViewItem->GetLabel())))
			{
				FilteredTreeViewItems.Add(TreeViewItem);
			}
		}
	}

	TreeViewWidget->RequestTreeRefresh();
}

void SWebAPITreeView::GetTreeItemChildren(FTreeItemType InObject, TArray<FTreeItemType>& OutChildren)
{
	if(!InObject.IsValid() || !InObject->IsValid())
	{
		return;
	}

	InObject->GetChildren(OutChildren);
}

void SWebAPITreeView::OnFilterTextChanged(const FText& InSearchText)
{
	FilterText = InSearchText;
	FilteredTreeViewItems.Reset();
	TextFilter->SetFilterText(FilterText);
 
	Refresh();
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<IWebAPIViewModel>(TSharedPtr<IWebAPIViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InViewModel.IsValid());

	const FName ViewModelTypeName = InViewModel->GetViewModelTypeName();

	if(ViewModelTypeName == IWebAPIViewModel::NAME_Service)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIServiceViewModel>(InViewModel), InOwnerTable);
	}
	
	if(ViewModelTypeName == IWebAPIViewModel::NAME_Operation)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIOperationViewModel>(InViewModel), InOwnerTable);
	}

	if(ViewModelTypeName == IWebAPIViewModel::NAME_OperationRequest)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIOperationRequestViewModel>(InViewModel), InOwnerTable);
	}

	if(ViewModelTypeName == IWebAPIViewModel::NAME_OperationParameter)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIOperationParameterViewModel>(InViewModel), InOwnerTable);
	}

	if(ViewModelTypeName == IWebAPIViewModel::NAME_OperationResponse)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIOperationResponseViewModel>(InViewModel), InOwnerTable);
	}
	
	if(ViewModelTypeName == IWebAPIViewModel::NAME_Model)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIModelViewModel>(InViewModel), InOwnerTable);
	}
	
	if(ViewModelTypeName == IWebAPIViewModel::NAME_Property)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIPropertyViewModel>(InViewModel), InOwnerTable);
	}

	if(ViewModelTypeName == IWebAPIViewModel::NAME_Enum)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIEnumViewModel>(InViewModel), InOwnerTable);
	}
	
	if(ViewModelTypeName == IWebAPIViewModel::NAME_EnumValue)
	{
		return GenerateRow(StaticCastSharedPtr<FWebAPIEnumValueViewModel>(InViewModel), InOwnerTable);
	}

	if(ViewModelTypeName == IWebAPIViewModel::NAME_Parameter)
	{
		return GenerateRow<FWebAPIParameterViewModel>(StaticCastSharedPtr<FWebAPIParameterViewModel>(InViewModel), InOwnerTable);
	}

	TSharedRef<STableRow<FTreeItemType>> Row =
		SNew(STableRow<FTreeItemType>, InOwnerTable)
		[
			SNew(STextBlock)
			.Text(InViewModel->GetLabel())
		];

	return Row;
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIServiceViewModel>(TSharedPtr<FWebAPIServiceViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaServiceRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIOperationViewModel>(TSharedPtr<FWebAPIOperationViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaOperationRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIOperationRequestViewModel>(TSharedPtr<FWebAPIOperationRequestViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaOperationRequestRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIOperationParameterViewModel>(TSharedPtr<FWebAPIOperationParameterViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaOperationParameterRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIOperationResponseViewModel>(TSharedPtr<FWebAPIOperationResponseViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaOperationResponseRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIModelViewModel>(TSharedPtr<FWebAPIModelViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaModelRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIPropertyViewModel>(TSharedPtr<FWebAPIPropertyViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaPropertyRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIEnumViewModel>(TSharedPtr<FWebAPIEnumViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaEnumRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIEnumValueViewModel>(TSharedPtr<FWebAPIEnumValueViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaEnumValueRow, InViewModel.ToSharedRef(), InOwnerTable);
}

template <>
TSharedRef<ITableRow> SWebAPITreeView::GenerateRow<FWebAPIParameterViewModel>(TSharedPtr<FWebAPIParameterViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SWebAPISchemaParameterRow, InViewModel.ToSharedRef(), InOwnerTable);
}

#undef LOCTEXT_NAMESPACE
