// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"

class FTextFilterExpressionEvaluator;
class FWebAPIEnumValueViewModel;
class FWebAPIEnumViewModel;
class FWebAPIModelViewModel;
class FWebAPIOperationParameterViewModel;
class FWebAPIOperationRequestViewModel;
class FWebAPIOperationResponseViewModel;
class FWebAPIOperationViewModel;
class FWebAPIParameterViewModel;
class FWebAPIPropertyViewModel;
class FWebAPIServiceViewModel;
class IWebAPIViewModel;
class STableViewBase;
template <typename ItemType> class STreeView;

class IDetailLayoutBuilder;

/** A widget containing either a tree of services or models. */
class WEBAPIEDITOR_API SWebAPITreeView : public SCompoundWidget
{
public:
	using FTreeItemType = TSharedPtr<IWebAPIViewModel>;

	using FOnSelectionChanged  = typename TSlateDelegates<FTreeItemType>::FOnSelectionChanged;
	
	SLATE_BEGIN_ARGS(SWebAPITreeView)
		: _OnSelectionChanged()
	{ }
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FText& InLabel, const TArray<FTreeItemType>& InTreeViewItems = {});

	/** Refresh the tree view. If new items are provided, they'll be replaced for this widget. */
	void Refresh(const TArray<TSharedPtr<IWebAPIViewModel>>& InTreeViewModels = {});

	/** Refresh the tree view. If new items are provided, they'll be replaced for this widget. */
	template <typename ViewModelType>
	void Refresh(const TArray<TSharedPtr<ViewModelType>>& InTreeViewModels = {});

private:
	/** Appends children of the provided Object to OutChildren. */
	void GetTreeItemChildren(FTreeItemType InObject, TArray<FTreeItemType>& OutChildren);

	/** Called when the associated search input changes. */
	void OnFilterTextChanged(const FText& InSearchText);

	/** Makes the row widget for the provided ViewModel. */
	template <typename ViewModelType>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<ViewModelType> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<IWebAPIViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIServiceViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIOperationViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIOperationRequestViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIOperationParameterViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIOperationResponseViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);
	
	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIModelViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIPropertyViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIEnumViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIEnumValueViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	template <>
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FWebAPIParameterViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

private:
	/** TreeView Widget. */
	TSharedPtr<STreeView<FTreeItemType>> TreeViewWidget;

	/** TreeView model context. */
	TArray<FTreeItemType> TreeViewItems;

	/** TreeView model context, filtered according to FilterText. */
	TArray<FTreeItemType> FilteredTreeViewItems;

	/** Current filter text, if any. */
	FText FilterText;

	/** The filter itself, if any. */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
};

template <typename ViewModelType>
void SWebAPITreeView::Refresh(const TArray<TSharedPtr<ViewModelType>>& InTreeViewModels)
{
	TArray<TSharedPtr<IWebAPIViewModel>> CastTreeViewModels;
	Algo::Transform(InTreeViewModels, CastTreeViewModels, [](const TSharedPtr<ViewModelType>& InItem)
	{
		return InItem;
	});
	Refresh(MoveTemp(CastTreeViewModels));
}
