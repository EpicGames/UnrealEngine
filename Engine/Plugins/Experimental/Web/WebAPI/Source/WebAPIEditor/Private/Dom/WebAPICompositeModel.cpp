// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPICompositeModel.h"
#include "Dom/WebAPISchema.h"
#include "Dom/WebAPIType.h"

namespace UE::WebAPI::WebAPIModelCompositionType
{
	const FString& ToString(const EWebAPIModelCompositionType& InEnumValue)
	{
		static TMap<EWebAPIModelCompositionType, FString> EnumNames = {
			{ EWebAPIModelCompositionType::Single, TEXT("Single") },
			{ EWebAPIModelCompositionType::Multiple, TEXT("Multiple") },
			{ EWebAPIModelCompositionType::All, TEXT("All") }
		};
 
		return EnumNames[InEnumValue];
	}

	const EWebAPIModelCompositionType& FromString(const FString& InStringValue)
	{
		static TMap<FString, EWebAPIModelCompositionType> EnumValues = {
			{ TEXT("Single"), EWebAPIModelCompositionType::Single },
			{ TEXT("Multiple"), EWebAPIModelCompositionType::Multiple },
			{ TEXT("All"), EWebAPIModelCompositionType::All },
		};

		return EnumValues[InStringValue];
	}
};

FString UWebAPICompositeModel::GetSortKey() const
{
	return Name.ToString(true);
}

void UWebAPICompositeModel::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);

	if(Name.HasTypeInfo() && !Name.TypeInfo->bIsBuiltinType)
	{
		Name.TypeInfo->Namespace = InNamespace;
	}
}
