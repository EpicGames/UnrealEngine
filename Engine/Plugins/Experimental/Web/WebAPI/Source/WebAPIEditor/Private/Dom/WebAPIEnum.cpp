// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIEnum.h"

#include "Dom/WebAPIType.h"

TObjectPtr<UWebAPIEnumValue> UWebAPIEnum::AddValue()
{
	return Values.Add_GetRef(NewObject<UWebAPIEnumValue>(this));
}

FString UWebAPIEnum::GetSortKey() const
{
	return Name.ToString(true);
}

void UWebAPIEnum::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);

	if(Name.HasTypeInfo() && !Name.TypeInfo->bIsBuiltinType)
	{
		Name.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIEnum::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	Super::Visit(InVisitor);

	for(const TObjectPtr<UWebAPIEnumValue>& Value : Values)
	{
		Value->Visit(InVisitor);		
	}
}

bool UWebAPIEnum::ToJson(TSharedPtr<FJsonObject>& OutJson)
{
	return IWebAPISchemaObjectInterface::ToJson(OutJson);
}

FString UWebAPIEnum::GetDefaultValue(bool bQualified) const
{
	if(!DefaultValue.IsEmpty())
	{
		return bQualified
		? Name.ToString() + TEXT("::") + DefaultValue
		: DefaultValue;
	}

	const FString Prefix = TEXT("EV"); // "EV" - Enum Value
	return !Values.IsEmpty() ?
		bQualified
		? Name.ToString() + TEXT("::") + Values[0]->Name.ToMemberName(Prefix)
		: Values[0]->Name.ToMemberName(Prefix)
	: TEXT("");
}

void UWebAPIEnum::BindToTypeInfo()
{
	Super::BindToTypeInfo();

	check(Name.HasTypeInfo());

	if(!Name.TypeInfo->bIsBuiltinType && Name.TypeInfo->GetModel() == nullptr)
	{
		Name.TypeInfo->SetModel(this);
		if(Name.TypeInfo->DefaultValue.IsEmpty())
		{
			Name.TypeInfo->DefaultValue = GetDefaultValue();
		}
	}
}

#if WITH_EDITOR
void UWebAPIEnum::SetCodeText(const FString& InCodeText)
{
	GeneratedCodeText = InCodeText;
}

void UWebAPIEnum::AppendCodeText(const FString& InCodeText)
{
	GeneratedCodeText += TEXT("\n") + InCodeText;
}
#endif
