// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIModel.h"

#include "WebAPIDefinition.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIType.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "WebAPIModel"

bool UWebAPIProperty::IsRequired() const
{
	return bIsRequired;
}

void UWebAPIProperty::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		Type.TypeInfo->Namespace = InNamespace;
	}
}

FString UWebAPIProperty::GetDefaultValue(bool bQualified) const
{
	if(!DefaultValue.IsEmpty())
	{
		return DefaultValue;
	}
	
	if(!bIsArray && Type.HasTypeInfo() && !Type.TypeInfo->DefaultValue.IsEmpty())
	{
		return Type.TypeInfo->GetDefaultValue(bQualified);
	}

	UWebAPIDefinition* OwningDefinition = GetTypedOuter<UWebAPIDefinition>();
	check(OwningDefinition);
	
	if(Type.HasTypeInfo())
	{
		if(const UObject* Model = Type.TypeInfo->GetModel())
		{
			if(const UWebAPIEnum* Enum = Cast<UWebAPIEnum>(Model))
			{
				return Enum->GetDefaultValue(bQualified);
			}
		}
	}

	// All generated enums have an "_Unset" value
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType && Type.TypeInfo->IsEnum())
	{
		if(bQualified)
		{
			return Type.TypeInfo.ToString() + TEXT("::") + OwningDefinition->GetProviderSettings().GetUnsetEnumValueName();
		}
		else
		{
			return OwningDefinition->GetProviderSettings().GetUnsetEnumValueName();
		}
	}

	return TEXT("");
}

#if WITH_EDITOR
EDataValidationResult UWebAPIProperty::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	if(Name.ToString(true).IsEmpty())
	{
		Context.AddError(LOCTEXT("Missing_Name", "Property missing name"));
		ValidationResult = EDataValidationResult::Invalid;
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

FString UWebAPIModel::GetSortKey() const
{
	return Name.ToString(true);
}

void UWebAPIModel::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);
	
	if(Name.HasTypeInfo())
	{
		Name.TypeInfo->Namespace = InNamespace;
	}

	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		Type.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIModel::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	Super::Visit(InVisitor);

	for(const TObjectPtr<UWebAPIProperty>& Property : Properties)
	{
		Property->Visit(InVisitor);
	}
}

void UWebAPIModel::BindToTypeInfo()
{
	Super::BindToTypeInfo();

	check(Name.HasTypeInfo());

	if(!Name.TypeInfo->bIsBuiltinType && Name.TypeInfo->GetModel() == nullptr)
	{
		Name.TypeInfo->SetModel(this);
	}
}

#if WITH_EDITOR
void UWebAPIModel::SetCodeText(const FString& InCodeText)
{
	GeneratedCodeText = InCodeText;
}

void UWebAPIModel::AppendCodeText(const FString& InCodeText)
{
	GeneratedCodeText += TEXT("\n") + InCodeText;
}

EDataValidationResult UWebAPIModel::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	if(Name.ToString(true).IsEmpty())
	{
		Context.AddError(LOCTEXT("Missing_Model_Name", "Model missing name"));
		ValidationResult = EDataValidationResult::Invalid;
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
