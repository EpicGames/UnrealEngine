// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIOperation.h"

#include "Dom/WebAPIType.h"

#define LOCTEXT_NAMESPACE "WebAPIOperationParameter"

namespace UE::WebAPI::WebAPIParameterStorage
{
	const FString& ToString(const EWebAPIParameterStorage& InEnumValue)
	{
		static TMap<EWebAPIParameterStorage, FString> EnumNames = {
			{ EWebAPIParameterStorage::Path, TEXT("Path") },
			{ EWebAPIParameterStorage::Query, TEXT("Query") },
			{ EWebAPIParameterStorage::Header, TEXT("Header") },
			{ EWebAPIParameterStorage::Cookie, TEXT("Cookie") },
			{ EWebAPIParameterStorage::Body, TEXT("Body") }
		};
 
		return EnumNames[InEnumValue];
	}

	const EWebAPIParameterStorage& FromString(const FString& InStringValue)
	{
		static TMap<FString, EWebAPIParameterStorage> EnumValues = {
			{ TEXT("Path"), EWebAPIParameterStorage::Path },
			{ TEXT("Query"), EWebAPIParameterStorage::Query },
			{ TEXT("Header"), EWebAPIParameterStorage::Header },
			{ TEXT("Cookie"), EWebAPIParameterStorage::Cookie },
			{ TEXT("Body"), EWebAPIParameterStorage::Body }
		};

		return EnumValues[InStringValue];
	}
}

bool UWebAPIOperationParameter::IsRequired() const
{
	// if Storage == Path, bIsRequired = always false
	return Storage == EWebAPIParameterStorage::Path || Super::IsRequired();
}

void UWebAPIOperationParameter::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		Type.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIOperationRequest::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);
	if(Type.HasTypeInfo() && !Type.TypeInfo->bIsBuiltinType)
	{
		Type.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIOperationRequest::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	Super::Visit(InVisitor);

	for(const TObjectPtr<UWebAPIOperationParameter>& Parameter : Parameters)
	{
		Parameter->Visit(InVisitor);		
	}
}

UWebAPIOperation::UWebAPIOperation()
{
	Request = CreateDefaultSubobject<UWebAPIOperationRequest>(TEXT("Request"));
}

void UWebAPIOperation::SetNamespace(const FString& InNamespace)
{
	check(Name.HasTypeInfo());

	if(Name.HasTypeInfo() && !Name.TypeInfo->bIsBuiltinType)
	{
		Name.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIOperation::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	IWebAPISchemaObjectInterface::Visit(InVisitor);
	
	Request->Visit(InVisitor);
	for(const TObjectPtr<UWebAPIOperationResponse>& Response : Responses)
	{
		Response->Visit(InVisitor);		
	}

	for(const TObjectPtr<UWebAPIOperationError>& Error : Errors)
	{
		Error->Visit(InVisitor);		
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UWebAPIOperation::BindToTypeInfo()
{
	check(Name.HasTypeInfo());

	if(!Name.TypeInfo->GetModel())
	{
		Name.TypeInfo->SetModel(this);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
void UWebAPIOperation::SetCodeText(const FString& InCodeText)
{
	GeneratedCodeText = InCodeText;
}

void UWebAPIOperation::AppendCodeText(const FString& InCodeText)
{
	GeneratedCodeText += TEXT("\n") + InCodeText;
}

EDataValidationResult UWebAPIOperation::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = Request->IsDataValid(Context);

	for(const TObjectPtr<UWebAPIOperationResponse>& Response : Responses)
	{
		ValidationResult = CombineDataValidationResults(Response->IsDataValid(Context), ValidationResult);
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
