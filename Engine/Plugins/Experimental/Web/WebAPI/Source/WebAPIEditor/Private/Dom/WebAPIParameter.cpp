// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIParameter.h"

#if WITH_EDITOR
void UWebAPIParameter::SetCodeText(const FString& InCodeText)
{
	GeneratedCodeText = InCodeText;
}

void UWebAPIParameter::AppendCodeText(const FString& InCodeText)
{
	GeneratedCodeText += TEXT("\n") + InCodeText;
}
#endif
