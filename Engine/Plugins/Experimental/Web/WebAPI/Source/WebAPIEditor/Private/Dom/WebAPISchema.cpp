// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPISchema.h"

#include "Dom/WebAPIType.h"
#include "IWebAPIEditorModule.h"
#include "WebAPIDefinition.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPITypeRegistry.h"
#include "WebAPIEditorUtilities.h"

#define LOCTEXT_NAMESPACE "WebAPISchema"

FString IWebAPISchemaObjectInterface::GetSortKey() const
{
	return TEXT("");
}

void IWebAPISchemaObjectInterface::SetNamespace(const FString& InNamespace)
{
	Visit([this, &InNamespace](IWebAPISchemaObjectInterface* InSchemaObject)
	{
		if(InSchemaObject != this)
		{
			InSchemaObject->SetNamespace(InNamespace);	
		}
	});
}

void IWebAPISchemaObjectInterface::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	IWebAPISchemaObjectInterface* Self = this;
	InVisitor(Self);
}

bool IWebAPISchemaObjectInterface::ToJson(TSharedPtr<FJsonObject>& OutJson)
{
	return false;
}

void IWebAPISchemaObjectInterface::SetCodeText(const FString& InCodeText) { }

void IWebAPISchemaObjectInterface::AppendCodeText(const FString& InCodeText) { }

FWebAPINameInfo::FWebAPINameInfo(const FString& InName, const FString& InJsonName, const FWebAPITypeNameVariant& InTypeInfo)
	: Name(InName)
	, JsonName(InJsonName)
{
	if(InTypeInfo == IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Boolean)
	{
		Prefix = TEXT("b");
	}
}

FString FWebAPINameInfo::ToString(bool bInJustName) const
{
	if(bInJustName)
	{
		return Name;
	}
	
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("BoolPrefix"), Name.Equals(TEXT("bool"), ESearchCase::IgnoreCase) ? TEXT("b") : TEXT(""));
	FormatArgs.Add(TEXT("Prefix"), Prefix);
	FormatArgs.Add(TEXT("Name"), Name);

	return FString::Format(TEXT("{BoolPrefix}{Prefix}{Name}"), FormatArgs);
}

FString FWebAPINameInfo::ToMemberName(const FString& InPrefix) const
{
	return UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(Name, InPrefix);
}

FString FWebAPINameInfo::GetDisplayName() const
{
	return ToString(true);
}

FString FWebAPINameInfo::GetJsonName() const
{
	return JsonName.IsEmpty() ? Name : JsonName;	
}

FWebAPITypeNameVariant::FWebAPITypeNameVariant(const TObjectPtr<UWebAPITypeInfo>& InTypeName)
	: TypeInfo(InTypeName)
{
}

FWebAPITypeNameVariant::FWebAPITypeNameVariant(const FString& InStringName)
	: TypeString(InStringName)
{
}

FWebAPITypeNameVariant::FWebAPITypeNameVariant(const TCHAR* InStringName)
	: TypeString(InStringName)
{
}

bool FWebAPITypeNameVariant::HasTypeInfo() const
{
	if(TypeInfo.IsValid() || !TypeInfo.IsNull())
	{
		UWebAPITypeInfo* Resolved = TypeInfo.LoadSynchronous();
		return Resolved != nullptr;
	}
	return false;;
}

bool FWebAPITypeNameVariant::IsValid() const
{
	return HasTypeInfo() || !TypeString.IsEmpty();
}

FString FWebAPITypeNameVariant::ToMemberName(const FString& InPrefix) const
{
	const FString TypeNameString = ToString(true);
	return TypeNameString.IsEmpty() ? TEXT("") : UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(TypeNameString, InPrefix);
}

FString FWebAPITypeNameVariant::ToString(bool bInJustName) const
{
	FString Result;
	if(HasTypeInfo())
	{
		Result = TypeInfo->ToString(bInJustName);		
	}
	return Result.IsEmpty() ? TypeString : Result;
}

FString FWebAPITypeNameVariant::GetDisplayName() const
{
	FString Result;
	if(HasTypeInfo())
	{
		Result = TypeInfo->GetDisplayName();		
	}
	return Result.IsEmpty() ? TypeString : Result;
}

FString FWebAPITypeNameVariant::GetJsonName() const
{
	return HasTypeInfo()
			? TypeInfo->GetJsonName()
			: TypeString;
}

FWebAPINameVariant::FWebAPINameVariant(const FWebAPINameInfo& InNameInfo)
	: NameInfo(InNameInfo)
{
	
}

FWebAPINameVariant::FWebAPINameVariant(const FString& InStringName)
	: NameString(InStringName)
{
	if(HasNameInfo())
	{
		NameInfo.Name = InStringName;
	}
}

FWebAPINameVariant::FWebAPINameVariant(const TCHAR* InStringName)
	: NameString(InStringName)
{
	if(HasNameInfo())
	{
		NameInfo.Name = InStringName;
	}
}

bool FWebAPINameVariant::HasNameInfo() const
{
	return !NameInfo.Name.IsEmpty() || !NameInfo.JsonName.IsEmpty();
}

bool FWebAPINameVariant::IsValid() const
{
	return HasNameInfo() || !NameString.IsEmpty();
}

FString FWebAPINameVariant::ToString(bool bInJustName) const
{
	if(HasNameInfo() && !NameInfo.Name.IsEmpty())
	{
		return bInJustName ? NameInfo.Name : NameInfo.ToString(bInJustName);		
	}
	return NameString;
}

FString FWebAPINameVariant::ToMemberName(const FString& InPrefix) const
{
	return HasNameInfo()
		? NameInfo.ToMemberName(InPrefix)
		: UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(NameString, InPrefix);
}

FString FWebAPINameVariant::GetDisplayName() const
{
	return HasNameInfo()
			? NameInfo.GetDisplayName()
			: NameString;
}

FString FWebAPINameVariant::GetJsonName() const
{
	return HasNameInfo()
			? NameInfo.GetJsonName()
			: NameString;
}

FWebAPINameVariant& FWebAPINameVariant::operator=(const FString& InStringName)
{
	NameString = InStringName;
	
	if(HasNameInfo())
	{
		NameInfo.Name = InStringName;
	}
	
	return *this;	
}

void UWebAPIModelBase::SetNamespace(const FString& InNamespace)
{
	IWebAPISchemaObjectInterface::SetNamespace(InNamespace);
}

void UWebAPIModelBase::BindToTypeInfo() { }

#if WITH_EDITOR
EDataValidationResult UWebAPIModelBase::IsDataValid(FDataValidationContext& Context) const
{
	return EDataValidationResult::Valid;
}
#endif // WITH_EDITOR

UWebAPISchema::UWebAPISchema()
{
	TypeRegistry = CreateDefaultSubobject<UWebAPITypeRegistry>(TEXT("TypeRegistry"));
}

UWebAPIEnum* UWebAPISchema::AddEnum(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
{
	check(InTypeInfo);

	const TObjectPtr<UWebAPIModelBase>& AddedModel = Models.Add_GetRef(NewObject<UWebAPIEnum>(this));

	const TObjectPtr<UWebAPIEnum> AddedEnum = Cast<UWebAPIEnum>(AddedModel);
	AddedEnum->Name = InTypeInfo;

	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ClassName"), UWebAPIEnum::StaticClass()->GetName());
	FormatArgs.Add(TEXT("EnumName"), InTypeInfo->ToString(true));
	
	const FName EnumObjectName = MakeUniqueObjectName(this, UWebAPIEnum::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{EnumName}"), FormatArgs)));
	AddedEnum->Rename(*EnumObjectName.ToString(), AddedEnum->GetOuter());
	
#if WITH_WEBAPI_DEBUG
	AddedEnum->Name.TypeInfo->DebugString += TEXT(">AddEnum");
#endif
	
	// AddEnum implies it's a generated, not-builtin type
	AddedEnum->bGenerate = true;

	if(InTypeInfo)
	{
		const FText LogMessage = FText::FormatNamed(
		LOCTEXT("AddedEnumWithName", "Added a new Enum \"{EnumName}\"."),
			TEXT("EnumName"), FText::FromString(AddedEnum->Name.ToString(true)));
		GetMessageLog()->LogInfo(LogMessage, UWebAPISchema::LogName);
	}
	else
	{
		GetMessageLog()->LogInfo(LOCTEXT("AddedEnum", "Added a new (unnamed) Enum."), UWebAPISchema::LogName);
	}

	return Cast<UWebAPIEnum>(AddedModel);
}

UWebAPIParameter* UWebAPISchema::AddParameter(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
{
	check(InTypeInfo);

	const TObjectPtr<UWebAPIModelBase>& AddedModel = Models.Add_GetRef(NewObject<UWebAPIParameter>(this));

	const TObjectPtr<UWebAPIParameter> AddedParameter = Cast<UWebAPIParameter>(AddedModel);
	AddedParameter->Name = InTypeInfo;

	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ClassName"), UWebAPIParameter::StaticClass()->GetName());
	FormatArgs.Add(TEXT("ParameterName"), InTypeInfo->ToString(true));
	
	const FName ParameterObjectName = MakeUniqueObjectName(this, UWebAPIParameter::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{ParameterName}"), FormatArgs)));
	AddedParameter->Rename(*ParameterObjectName.ToString(), AddedParameter->GetOuter());
	
#if WITH_WEBAPI_DEBUG
	AddedParameter->Name.TypeInfo->DebugString += TEXT(">AddParameter");
#endif
	
	// AddParameter implies it's a generated, not-builtin type
	AddedParameter->bGenerate = true;

	if(InTypeInfo)
	{
		const FText LogMessage = FText::FormatNamed(
		LOCTEXT("AddedParameterWithName", "Added a new Parameter \"{ParameterName}\"."),
			TEXT("ParameterName"), FText::FromString(AddedParameter->Name.ToString(true)));
		GetMessageLog()->LogInfo(LogMessage, UWebAPISchema::LogName);
	}
	else
	{
		GetMessageLog()->LogInfo(LOCTEXT("AddedParameter", "Added a new (unnamed) Parameter."), UWebAPISchema::LogName);
	}

	return Cast<UWebAPIParameter>(AddedModel);
}

TObjectPtr<UWebAPIService> UWebAPISchema::GetOrMakeService(const FString& InName)
{
	if(const TObjectPtr<UWebAPIService>* FoundService = Services.Find(InName))
	{
		return *FoundService;		
	}

	const FText LogMessage = FText::FormatNamed(
		LOCTEXT("AddedService", "Added a new Service \"{ServiceName}\"."),
		TEXT("ServiceName"), FText::FromString(InName));
	GetMessageLog()->LogInfo(LogMessage, UWebAPISchema::LogName);

	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ClassName"), UWebAPIService::StaticClass()->GetName());
	FormatArgs.Add(TEXT("ServiceName"), InName);

	const FName ServiceObjectName = MakeUniqueObjectName(this, UWebAPIService::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{ServiceName}"), FormatArgs)));

	ensure(IsInGameThread());
	Services.Add(InName, NewObject<UWebAPIService>(this, ServiceObjectName));
	
	const TObjectPtr<UWebAPIService>& AddedService = Services[InName];
	check(IsValid(AddedService));
	AddedService->Name = TypeRegistry->GetOrMakeGeneratedType(EWebAPISchemaType::Service, InName, InName, IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object); 
	auto* Test = TypeRegistry->FindGeneratedType(EWebAPISchemaType::Service, InName);
	AddedService->Name.TypeInfo = *Test;
	check(AddedService->Name.HasTypeInfo());

	check(Test);
	check(IsValid(AddedService));
	check(AddedService->Name.HasTypeInfo());
	AddedService->Name.TypeInfo->SetName(InName);
	AddedService->Name.TypeInfo->Prefix = TEXT("");
	AddedService->Name.TypeInfo->SchemaType = EWebAPISchemaType::Service;

#if WITH_WEBAPI_DEBUG
	AddedService->Name.TypeInfo->DebugString += TEXT(">GetOrMakeService");
#endif

	return AddedService;
}

void UWebAPISchema::Clear()
{
	APIName.Empty();
	Host.Empty();
	BaseUrl.Empty();
	DateTimeFormat.Empty();
	Version.Empty();
	Services.Empty(Services.Num());
	Models.Empty(Models.Num());
}

const TSharedPtr<FWebAPIMessageLog>& UWebAPISchema::GetMessageLog() const
{
	UWebAPIDefinition* OuterDefinition = GetTypedOuter<UWebAPIDefinition>();
	check(OuterDefinition);

	return OuterDefinition->GetMessageLog();
}

void UWebAPISchema::Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	IWebAPISchemaObjectInterface::Visit(InVisitor);

	for(const TPair<FString, TObjectPtr<UWebAPIService>>& Service : Services)
	{
		Service.Value->Visit(InVisitor);		
	}

	for(const TObjectPtr<UWebAPIModelBase>& Model : Models)
	{
		Model->Visit(InVisitor);
	}
}

#if WITH_EDITOR
EDataValidationResult UWebAPISchema::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = EDataValidationResult::Valid;

	for(const TPair<FString, TObjectPtr<UWebAPIService>>& ServicePair : Services)
	{
		ValidationResult = CombineDataValidationResults(ServicePair.Value->IsDataValid(Context), ValidationResult);
	}
	
	for(const TObjectPtr<UWebAPIModelBase>& Model : Models)
	{
		ValidationResult = CombineDataValidationResults(Model->IsDataValid(Context), ValidationResult);
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
