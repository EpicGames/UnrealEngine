// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIService.h"

#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIType.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "WebAPIService"

void UWebAPIService::SetNamespace(const FString& InNamespace)
{
	Super::SetNamespace(InNamespace);
	check(Name.HasTypeInfo());

	if(Name.HasTypeInfo() && !Name.TypeInfo->bIsBuiltinType)
	{
		Name.TypeInfo->Namespace = InNamespace;
	}
}

void UWebAPIService::Visit(const TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor)
{
	Super::Visit(InVisitor);

	for(const TObjectPtr<UWebAPIOperation>& Operation : Operations)
	{
		Operation->Visit(InVisitor);
	}
}

#if WITH_EDITOR
EDataValidationResult UWebAPIService::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = Super::IsDataValid(Context);

	for(const TObjectPtr<UWebAPIOperation>& Operation : Operations)
	{
		ValidationResult = CombineDataValidationResults(Operation->IsDataValid(Context), ValidationResult);
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
