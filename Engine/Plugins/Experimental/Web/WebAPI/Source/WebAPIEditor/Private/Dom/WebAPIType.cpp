// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPIType.h"

#include "Dom/WebAPISchema.h"
#include "WebAPIEditorUtilities.h"
#include "Dom/WebAPITypeRegistry.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "WebAPITypeInfo"

namespace UE::WebAPI::WebAPISchemaType
{
	const FString& ToString(EWebAPISchemaType SchemaType)
	{
		static TMap<EWebAPISchemaType, FString> SchemaTypeToString = {
			{EWebAPISchemaType::Model, TEXT("Model")},
			{EWebAPISchemaType::Service, TEXT("Service")},
			{EWebAPISchemaType::Operation, TEXT("Operation")},
			{EWebAPISchemaType::Property, TEXT("Property")},
			{EWebAPISchemaType::Parameter, TEXT("Parameter")},
			{EWebAPISchemaType::Unspecified, TEXT("Unspecified")}
		};

		return SchemaTypeToString[SchemaType];
	}
}

bool UWebAPITypeInfo::IsEnum() const
{
	return DeclarationType != NAME_None && (DeclarationType == NAME_Enum || DeclarationType == NAME_EnumProperty || DeclarationType == NAME_UserDefinedEnum);
}

TObjectPtr<UWebAPITypeInfo> UWebAPITypeInfo::Duplicate(const TObjectPtr<UWebAPITypeRegistry>& InTypeRegistry) const
{
	check(InTypeRegistry);
	const TObjectPtr<UWebAPITypeInfo> DuplicatedObject = InTypeRegistry->GetOrMakeGeneratedType(this->SchemaType, this->Name, this->JsonName, this);
	if(DuplicatedObject->DebugString.IsEmpty())
	{
		DuplicatedObject->DebugString +=	TEXT("Duplicated from another type.");
	}
	return DuplicatedObject;
}

FString UWebAPITypeInfo::ToString(bool bInJustName) const
{
	// Is enum, but name unknown
	if(Name.IsEmpty() && IsEnum())
	{
		return TEXT("");	
	}

	if(bInJustName)
	{
		return Name.IsEmpty() ? TEXT("") : Name;
	}
	
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("Prefix"), Prefix);
	FormatArgs.Add(TEXT("Name"), Name);
	FormatArgs.Add(TEXT("Namespace"), Namespace);
	FormatArgs.Add(TEXT("Suffix"), Suffix);

	FString TypeStr = Namespace.IsEmpty() || bIsBuiltinType
			? FString::Format(TEXT("{Prefix}{Name}{Suffix}"), FormatArgs)
			: FString::Format(TEXT("{Prefix}{Namespace}_{Name}{Suffix}"), FormatArgs);

	return TypeStr;
}

FString UWebAPITypeInfo::ToMemberName(const FString& InPrefix) const
{
	return UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(ToString(true) + Suffix, InPrefix);
}

FString UWebAPITypeInfo::GetDisplayName() const
{
	return !DisplayName.IsEmpty() ? DisplayName : ToString(true);
}

FString UWebAPITypeInfo::GetJsonName() const
{
	return JsonName.IsEmpty() ? Name : JsonName;
}

FString UWebAPITypeInfo::GetDefaultValue(bool bQualified) const
{
	if(IsEnum() && bQualified)
	{
		return ToString() + TEXT("::") + DefaultValue;		
	}

	return DefaultValue;
}

void UWebAPITypeInfo::SetName(const FString& InName)
{
	if(Name == InName)
	{
		return;
	}
	
	Name = InName;
	if(GetName().Contains(TEXT("__")))
	{
		Rename(*UE::WebAPI::MakeTypeInfoName(GetOuter(), SchemaType, Name).ToString());
	}
}

void UWebAPITypeInfo::SetNested(const FWebAPITypeNameVariant& InNester)
{
	checkf(InNester.HasTypeInfo(), TEXT("A nested type must have a resolved TypeInfo as it's containing type."));

	bIsNested = true;
	ContainingType = InNester;
}

UObject* UWebAPITypeInfo::GetModel() const
{
	if (Model.IsNull())
	{
		return nullptr;
	}

	return Model.LoadSynchronous();
}

bool operator==(const UWebAPITypeInfo& A, const UWebAPITypeInfo& B)
{
	return (A.Name + A.Suffix) == (B.Name + B.Suffix)
		&& A.SchemaType == B.SchemaType
		&& A.bIsBuiltinType == B.bIsBuiltinType;
}

#if WITH_EDITOR
EDataValidationResult UWebAPITypeInfo::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	if(ToString(true).IsEmpty())
	{
		Context.AddError(LOCTEXT("Missing_Name", "Unnamed TypeInfo"));
		ValidationResult = EDataValidationResult::Invalid;
	}

	return ValidationResult;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
