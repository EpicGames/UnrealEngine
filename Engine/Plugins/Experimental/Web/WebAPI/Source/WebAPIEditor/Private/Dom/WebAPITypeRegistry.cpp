// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/WebAPITypeRegistry.h"

#include "Dom/WebAPIType.h"
#include "GraphEditorSettings.h"
#include "WebAPIEditorLog.h"
#include "Algo/Copy.h"
#include "Serialization/JsonTypes.h"

#define LOCTEXT_NAMESPACE "WebAPITypeRegistry"

const TObjectPtr<UWebAPITypeInfo>& UWebAPIStaticTypeRegistry::GetOrMakeBuiltinType(const FString& InName, const FString& InJsonName, const FSlateColor& InPinColor, const FString& InPrefix, FName InDeclarationType)
{
	if(const TObjectPtr<UWebAPITypeInfo>* FoundType = FindBuiltinType(InName))
	{
		return *FoundType;
	}

	UWebAPITypeInfo* TypeInfo = NewObject<UWebAPITypeInfo>(this, UE::WebAPI::MakeTypeInfoName(this, EWebAPISchemaType::Unspecified, InName));
	TypeInfo->Prefix = InPrefix;
	TypeInfo->SetName(InName);
	TypeInfo->JsonName = InJsonName;
	TypeInfo->bIsBuiltinType = true;
	TypeInfo->DeclarationType = InDeclarationType;
	TypeInfo->PinColor = InPinColor;

	if(TypeInfo->Name.IsEmpty())
	{
		// By default, unless otherwise specified (in Name), an inner array type is string
		if(InDeclarationType != NAME_None && !TypeInfo->IsEnum())
		{
			TypeInfo->Prefix = TEXT("F");
			TypeInfo->SetName(TEXT("String"));
		}
	}

	return BuiltinTypes.Add_GetRef(MoveTemp(TypeInfo));
}

const TObjectPtr<UWebAPITypeInfo>* UWebAPIStaticTypeRegistry::FindBuiltinType(const FString& InName)
{
	// Has a name, or no name but a container
	if(!InName.IsEmpty())
	{
		if(const TObjectPtr<UWebAPITypeInfo>* FoundType = BuiltinTypes.FindByPredicate(
			[&InName](const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
		{
			return InTypeInfo->Name == InName;
		}))
		{
			return FoundType;
		}
	}

	return nullptr;
}

void UWebAPIStaticTypeRegistry::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	InitializeTypes();
}

const UGraphEditorSettings* UWebAPIStaticTypeRegistry::GetGraphEditorSettings()
{
	return GetDefault<UGraphEditorSettings>();
}

void UWebAPIStaticTypeRegistry::InitializeTypes()
{
	static TMap<EJson, FName> EJson_EnumToString = {
		{ EJson::None, TEXT("None") },
		{ EJson::Null, TEXT("Null") },
		{ EJson::String, TEXT("String") },
		{ EJson::Boolean, TEXT("Bool") },
		{ EJson::Number, TEXT("Number") },
		{ EJson::Array, TEXT("Array") },
		{ EJson::Object, TEXT("Object") },
	};
	
	Nullptr = GetOrMakeBuiltinType(TEXT("nullptr"), {}, GetGraphEditorSettings()->ExecutionPinTypeColor);
	Nullptr->JsonType = EJson_EnumToString[EJson::Null];
	
	Auto = GetOrMakeBuiltinType(TEXT("auto"), {}, GetGraphEditorSettings()->WildcardPinTypeColor);
	Void = GetOrMakeBuiltinType(TEXT("void"), {}, GetGraphEditorSettings()->ExecutionPinTypeColor);

	String = GetOrMakeBuiltinType(TEXT("String"), {}, GetGraphEditorSettings()->StringPinTypeColor, TEXT("F"));
	String->JsonType = EJson_EnumToString[EJson::String];
	String->PrintFormatSpecifier = TEXT("s");
	String->DefaultValue = TEXT("TEXT(\"\")");

	Name = GetOrMakeBuiltinType(TEXT("Name"), {}, GetGraphEditorSettings()->NamePinTypeColor, TEXT("F"));
	Name->JsonType = EJson_EnumToString[EJson::String];
	Name->DefaultValue = TEXT("NAME_None");
	Name->PrintFormatSpecifier = TEXT("s");
	Name->PrintFormatExpression = TEXT("{Property}.ToString()");

	Text = GetOrMakeBuiltinType(TEXT("Text"), {}, GetGraphEditorSettings()->TextPinTypeColor, TEXT("F"));
	Text->JsonType = EJson_EnumToString[EJson::String];
	Text->PrintFormatSpecifier = TEXT("s");
	Text->DefaultValue = TEXT("FText::GetEmpty()");
	
	Char = GetOrMakeBuiltinType(TEXT("Char"), {}, GetGraphEditorSettings()->StringPinTypeColor, TEXT("F"));
	Char->JsonType = EJson_EnumToString[EJson::String];
	
	Boolean = GetOrMakeBuiltinType(TEXT("bool"), {}, GetGraphEditorSettings()->BooleanPinTypeColor);
	Boolean->JsonType = EJson_EnumToString[EJson::Boolean];
	Boolean->DefaultValue = TEXT("false");
	
	Float = GetOrMakeBuiltinType(TEXT("float"), {}, GetGraphEditorSettings()->FloatPinTypeColor);
	Float->DefaultValue = TEXT("0.0f");

	Double = GetOrMakeBuiltinType(TEXT("double"), {}, GetGraphEditorSettings()->DoublePinTypeColor);
	Double->DefaultValue = TEXT("0.0");

	Byte = GetOrMakeBuiltinType(TEXT("uint8"), {}, GetGraphEditorSettings()->BytePinTypeColor);
	Int16 = GetOrMakeBuiltinType(TEXT("int16"), {}, GetGraphEditorSettings()->IntPinTypeColor);
	Int32 = GetOrMakeBuiltinType(TEXT("int32"), {}, GetGraphEditorSettings()->IntPinTypeColor);
	Int64 = GetOrMakeBuiltinType(TEXT("int64"), {}, GetGraphEditorSettings()->Int64PinTypeColor);

	Float->JsonType = Double->JsonType = Byte->JsonType = Int16->JsonType = Int32->JsonType = Int64->JsonType = EJson_EnumToString[EJson::Number];
	Byte->DefaultValue = Int16->DefaultValue = Int32->DefaultValue = Int64->DefaultValue = TEXT("0");
	Float->PrintFormatSpecifier = Double->PrintFormatSpecifier = Byte->PrintFormatSpecifier = Int16->PrintFormatSpecifier = Int32->PrintFormatSpecifier = Int64->PrintFormatSpecifier = TEXT("d");

	FilePath = GetOrMakeBuiltinType(TEXT("FilePath"), {}, GetGraphEditorSettings()->StructPinTypeColor, TEXT("F"));
	FilePath->JsonType = EJson_EnumToString[EJson::String];
	FilePath->JsonPropertyToSerialize = TEXT("FilePath");
	FilePath->PrintFormatSpecifier = TEXT("s");
	
	DateTime = GetOrMakeBuiltinType(TEXT("DateTime"), {}, GetGraphEditorSettings()->StructPinTypeColor, TEXT("F"));
	DateTime->JsonType = EJson_EnumToString[EJson::String];
	DateTime->PrintFormatSpecifier = TEXT("s");
	DateTime->PrintFormatExpression = TEXT("{Property}.ToString()");

	Enum = GetOrMakeBuiltinType(TEXT(""), {}, GetGraphEditorSettings()->IndexPinTypeColor, TEXT("E"), NAME_Enum);
	Enum->JsonType = ToFromJsonType;
	Enum->bIsBuiltinType = true;
	Enum->PrintFormatSpecifier = TEXT("s");
	Enum->PrintFormatExpression = TEXT("UE::{Namespace}::ToString({Property})");

	Object = GetOrMakeBuiltinType(TEXT(""), {}, GetGraphEditorSettings()->ObjectPinTypeColor, TEXT("U"));
	Object->bIsBuiltinType = true;
	Object->PrintFormatSpecifier = TEXT("");

	WebAPIMessageResponse = GetOrMakeBuiltinType(TEXT("WebAPIMessageResponse"), {}, GetGraphEditorSettings()->StructPinTypeColor, TEXT("F"));
	WebAPIMessageResponse->PrintFormatSpecifier = TEXT("s");
	WebAPIMessageResponse->PrintFormatExpression = TEXT("{Property}.GetMessage()");
	WebAPIMessageResponse->IncludePaths.Add(TEXT("WebAPIMessageResponse.h"));

	JsonObject = GetOrMakeBuiltinType(TEXT("JsonObjectWrapper"), {}, GetGraphEditorSettings()->StructPinTypeColor, TEXT("F"));
	JsonObject->DisplayName = TEXT("JsonObject");
	JsonObject->JsonType = ToFromJsonObject;
	JsonObject->Modules.Add(TEXT("Json"));
	JsonObject->IncludePaths.Add(TEXT("JsonObjectWrapper.h"));
}

TObjectPtr<UWebAPITypeInfo> UWebAPITypeRegistry::GetOrMakeGeneratedType(
	const EWebAPISchemaType& InSchemaType,
	const FString& InName,
	const FString& InJsonName,
	const FString& InPrefix,
	FName InDeclarationType)
{
	ensureAlways(!InName.IsEmpty());
	
	if(const TObjectPtr<UWebAPITypeInfo>* FoundType = FindGeneratedType(InSchemaType, InName))
	{
		return *FoundType;
	}

	const FName TypeName = UE::WebAPI::MakeTypeInfoName(this, InSchemaType, InName);
	UWebAPITypeInfo* TypeInfo = NewObject<UWebAPITypeInfo>(this, TypeName);	
	TypeInfo->SchemaType = InSchemaType;
	TypeInfo->JsonName = InJsonName;
	TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
	TypeInfo->Prefix = InPrefix;
	TypeInfo->bIsBuiltinType = false;
	TypeInfo->DeclarationType = InDeclarationType;
	TypeInfo->SetName(InName);

#if WITH_WEBAPI_DEBUG
	UE_LOGF(LogWebAPIEditor, Display, "New Type Path: %ls", *TypeInfo->GetPathName());
	TypeInfo->DebugString += TEXT(">Created in GetOrMakeGeneratedType");
#endif

	FSlateColor PinColor = UWebAPIStaticTypeRegistry::GetGraphEditorSettings()->StructPinTypeColor;
	if(TypeInfo->DeclarationType != NAME_None)
	{
		if(TypeInfo->IsEnum())
		{
			PinColor = UWebAPIStaticTypeRegistry::GetGraphEditorSettings()->BytePinTypeColor;	
		}
	}
	
	TypeInfo->PinColor = PinColor;
	return GeneratedTypes.Add_GetRef(MoveTemp(TypeInfo));
}

TObjectPtr<UWebAPITypeInfo> UWebAPITypeRegistry::GetOrMakeGeneratedType(
	const EWebAPISchemaType& InSchemaType,
	const FString& InName,
	const FString& InJsonName,
	const TObjectPtr<const UWebAPITypeInfo>& InTemplateTypeInfo)
{
	ensureAlways(!InName.IsEmpty());

#if WITH_WEBAPI_DEBUG
	UE_LOGF(LogWebAPIEditor, Display, "Template Type Path: %ls", *InTemplateTypeInfo->GetPathName());	
#endif
	
	if(const TObjectPtr<UWebAPITypeInfo>* FoundType = FindGeneratedType(InSchemaType, InName))
	{
		return *FoundType;
	}

	const FName TypeName = UE::WebAPI::MakeTypeInfoName(this, InSchemaType, InName);
	GeneratedTypes.Emplace(DuplicateObject(InTemplateTypeInfo.Get(), this, TypeName));
	const TObjectPtr<UWebAPITypeInfo>& TypeInfo = GeneratedTypes.Last();

#if WITH_WEBAPI_DEBUG
	UE_LOGF(LogWebAPIEditor, Display, "Duplicate Type Path: %ls", *TypeInfo->GetPathName());
#endif

	TypeInfo->Name = InName;
	TypeInfo->SchemaType = InSchemaType;
	TypeInfo->JsonName = InJsonName;
	TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
	TypeInfo->bIsBuiltinType = false;

#if WITH_WEBAPI_DEBUG
	TypeInfo->DebugString += TEXT(">Created in GetOrMakeGeneratedType");
#endif

	FSlateColor PinColor = UWebAPIStaticTypeRegistry::GetGraphEditorSettings()->StructPinTypeColor;
	if(TypeInfo->DeclarationType != NAME_None)
	{
		if(TypeInfo->IsEnum())
		{
			PinColor = UWebAPIStaticTypeRegistry::GetGraphEditorSettings()->BytePinTypeColor;	
		}
	}
	
	TypeInfo->PinColor = PinColor;
	return TypeInfo;
}

const TObjectPtr<UWebAPITypeInfo>* UWebAPITypeRegistry::FindGeneratedType(const EWebAPISchemaType& InSchemaType, const FString& InName)
{
	if(!InName.IsEmpty())
	{
		if(const TObjectPtr<UWebAPITypeInfo>* FoundType = GeneratedTypes.FindByPredicate(
			[&InName, &InSchemaType](const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
		{
			return InTypeInfo->Name == InName
				&& InTypeInfo->SchemaType == InSchemaType;
		}))
		{
			return FoundType;
		}
	}

	return nullptr;
}

const TArray<TObjectPtr<UWebAPITypeInfo>>& UWebAPITypeRegistry::GetGeneratedTypes() const
{
	return GeneratedTypes;
}

TArray<TObjectPtr<UWebAPITypeInfo>>& UWebAPITypeRegistry::GetMutableGeneratedTypes()
{
	return GeneratedTypes;
}

void UWebAPITypeRegistry::Clear()
{
	// Same number of types expected, so allocate to allow
	GeneratedTypes.Empty(GeneratedTypes.Num());
}

bool UWebAPITypeRegistry::CheckAllNamed() const
{
	TArray<TObjectPtr<UWebAPITypeInfo>> UnnamedTypes;
	if(!CheckAllNamed(UnnamedTypes))
	{
		return false;
	}

	return true;
}

bool UWebAPITypeRegistry::CheckAllNamed(TArray<TObjectPtr<UWebAPITypeInfo>>& OutUnnamedTypes) const
{
	Algo::CopyIf(GeneratedTypes, OutUnnamedTypes, [](const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
	{
		return InTypeInfo->ToString(true).IsEmpty();
	});

	return OutUnnamedTypes.IsEmpty();
}

EDataValidationResult UWebAPITypeRegistry::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	for(const TObjectPtr<UWebAPITypeInfo>& GeneratedType : GeneratedTypes)
	{
		ValidationResult = CombineDataValidationResults(GeneratedType->IsDataValid(Context), EDataValidationResult::Valid);
	}

	return ValidationResult;
}

FName UE::WebAPI::MakeTypeInfoName(UObject* InOuter, const EWebAPISchemaType& InSchemaType, const FString& InName)
{
	FStringFormatNamedArguments NameArgs;
	NameArgs.Add(TEXT("ClassName"), UWebAPITypeInfo::StaticClass()->GetName());
	NameArgs.Add(TEXT("SchemaType"), UE::WebAPI::WebAPISchemaType::ToString(InSchemaType));
	NameArgs.Add(TEXT("TypeName"), InName.IsEmpty() ? UWebAPIStaticTypeRegistry::UnnamedTypeName : InName);

	FName Result;
	if(InSchemaType == EWebAPISchemaType::Unspecified)
	{
		Result = MakeUniqueObjectName(InOuter, UWebAPITypeInfo::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{TypeName}_"), NameArgs)));
	}
	else
	{
		Result = MakeUniqueObjectName(InOuter, UWebAPITypeInfo::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{SchemaType}_{TypeName}_"), NameArgs)));
	}

#if WITH_WEBAPI_DEBUG
	UE_LOGF(LogWebAPIEditor, Display, "MakeTypeName: %ls", *Result.ToString());
#endif
	
	return Result;
}

#undef LOCTEXT_NAMESPACE
