// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/WebAPIDefinitionFactory.h"

#include "Dom/WebAPIType.h"
#include "IWebAPIEditorModule.h"
#include "Subsystems/ImportSubsystem.h"
#include "WebAPIDefinition.h"
#include "WebAPIEditorLog.h"
#include "WebAPIEditorUtilities.h"
#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Algo/Partition.h"
#include "Async/Async.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPITypeRegistry.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/FindObjectReferencers.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#define LOCTEXT_NAMESPACE "UWebAPIDefinitionFactory"

UWebAPIDefinitionFactory::UWebAPIDefinitionFactory(): Super()
{
    SupportedClass = UWebAPIDefinition::StaticClass();
    bCreateNew = false;
    bEditorImport = true;
    bText = true;
    bEditAfterNew = true;

	// Increase the priority so that this Factory is considered before other Factories that handle Json
	++ImportPriority;
}

bool UWebAPIDefinitionFactory::CanImportWebAPI(const FString& InFileName, const FString& InFileContents)
{
	return false;
}

TFuture<bool> UWebAPIDefinitionFactory::ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents)
{
	return MakeFulfilledPromise<bool>(false).GetFuture();
}

bool AddMissingDefaultResponses(const UWebAPIDefinition* InDefinition, const TObjectPtr<UWebAPIOperation>& InOperation, TArray<TObjectPtr<UWebAPIOperationResponse>>& InOperationResponses)
{
	static TSet<uint32> DefaultResponseCodes = { 200, 404 };

	TSet<uint32> OperationResponseCodes;
	Algo::Transform(InOperationResponses, OperationResponseCodes, [](const TObjectPtr<UWebAPIOperationResponse>& InResponse)
	{
		return InResponse->Code;
	});

	const TSet<uint32> MissingResponses = DefaultResponseCodes.Difference(OperationResponseCodes);
	Algo::ForEach(MissingResponses, [&InOperation, &InDefinition, &InOperationResponses](const uint32& InResponseCode)
	{
		const FString ResponseTypeNameStr = FString::Printf(TEXT("%s_Response_%s"),
			*InOperation->Name.ToString(true),
			InResponseCode > 0 ? *FString::FormatAsNumber(InResponseCode) : TEXT("Default"));
		const FWebAPITypeNameVariant ResponseTypeName = InDefinition->GetWebAPISchema()->TypeRegistry->GetOrMakeGeneratedType(EWebAPISchemaType::Model, ResponseTypeNameStr, ResponseTypeNameStr, IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object);
		ResponseTypeName.TypeInfo->Prefix = TEXT("F");

		const TObjectPtr<UWebAPIOperationResponse> Response = NewObject<UWebAPIOperationResponse>(InOperation);
		Response->Name = ResponseTypeName;
		Response->Code = InResponseCode;
		Response->Description = EHttpResponseCodes::GetDescription(StaticCast<EHttpResponseCodes::Type>(InResponseCode)).ToString();
		Response->BindToTypeInfo();
		InOperationResponses.Add(Response);
	});
	
	return true;
}

TFuture<void> UWebAPIDefinitionFactory::PostImportWebAPI(UWebAPIDefinition* InDefinition)
{
	// Add missing (but required) responses, ie. 200, 404
	{
		TArray<TObjectPtr<UWebAPIOperation>> AllOperations;
		for(const TPair<FString, TObjectPtr<UWebAPIService>>& Service : InDefinition->GetWebAPISchema()->Services)
		{
			AllOperations.Append(Service.Value->Operations);			
		}

		for(TObjectPtr<UWebAPIOperation>& Operation : AllOperations)
		{
			AddMissingDefaultResponses(InDefinition, Operation, Operation->Responses);
		}
	}

	// Inject common pseudo-namespace
	InDefinition->GetWebAPISchema()->SetNamespace(InDefinition->GetGeneratorSettings().GetNamespace());

	// Replace all unresolved object types with JsonObject type
	TArray<TObjectPtr<UWebAPITypeInfo>>& GeneratedTypes = InDefinition->GetWebAPISchema()->TypeRegistry->GetMutableGeneratedTypes();
	const int32 FirstUnresolvedObjectIndex =
		Algo::Partition(
		GeneratedTypes.GetData(),
		GeneratedTypes.Num(),
		[](const TObjectPtr<UWebAPITypeInfo>& InGeneratedType)
		{
			// check if either built-in type, or Model property is set
			return InGeneratedType->bIsBuiltinType
			|| InGeneratedType->GetModel()
			|| InGeneratedType->SchemaType != EWebAPISchemaType::Model;
		});

	// Otherwise no unresolved types
	if(FirstUnresolvedObjectIndex < GeneratedTypes.Num())
	{
		const TObjectPtr<UWebAPITypeInfo> JsonObjectType = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->JsonObject;

		TArray<TObjectPtr<UObject>> UnresolvedTypes;
		TMap<UObject*, UObject*> Redirects;
		for(int32 Idx = FirstUnresolvedObjectIndex; Idx < GeneratedTypes.Num(); ++Idx)
		{
			UnresolvedTypes.Add(GeneratedTypes[Idx]);
			Redirects.Add(GeneratedTypes[Idx], JsonObjectType);
		}

		TArray<UObject*> Referencers;
		for (auto Referencer : TFindObjectReferencers<UObject>(UnresolvedTypes, nullptr, false, true))
		{
			Referencers.Add(Referencer.Value);
		}

		for (UObject* Referencer : Referencers)
		{
			FArchiveReplaceObjectRef<UObject>(Referencer, Redirects);
		}
	}

	// Remove stale types from TypeRegistry
	GeneratedTypes.RemoveAt(FirstUnresolvedObjectIndex, GeneratedTypes.Num() - FirstUnresolvedObjectIndex);

	FDataValidationContext Context;
	EDataValidationResult ValidationResult = InDefinition->IsDataValid(Context);

	TArray<FText> Warnings, Errors;
	Context.SplitIssues(Warnings, Errors);

	for (const FText& ValidationWarning : Warnings)
	{
		InDefinition->GetMessageLog()->LogWarning(ValidationWarning, LogName);
	}

	for (const FText& ValidationError : Errors)
	{
		InDefinition->GetMessageLog()->LogError(ValidationError, LogName);
	}

	if(ValidationResult == EDataValidationResult::Invalid)
	{
		InDefinition->GetMessageLog()->LogError(LOCTEXT("DataValidationFailed", "There was one or more errors validating the WebAPI InDefinition after reimport."), LogName);
	}

	return MakeFulfilledPromise<void>().GetFuture();
}

bool UWebAPIDefinitionFactory::IsValidFileExtension(const FString& InFileExtension) const { return false; }

bool UWebAPIDefinitionFactory::FactoryCanImport(const FString& InFileName)
{
	const FString FileContents = ReadFileContents(InFileName);
	return CanImportWebAPI(InFileName, FileContents);
}

UObject* UWebAPIDefinitionFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled)
{
	const FString FileExtension = FPaths::GetExtension(InFilename);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, *FileExtension);

	UWebAPIDefinition* NewAsset = NewObject<UWebAPIDefinition>(InParent, InClass, InName, InFlags | RF_Transactional | RF_MarkAsRootSet);
	if (!ensure(NewAsset))
	{
		return nullptr;
	}

	if(!NewAsset->AssetImportData)
	{
		NewAsset->AssetImportData = NewObject<UAssetImportData>();
	}
	
	FAssetImportInfo Info;
	Info.Insert(FAssetImportInfo::FSourceFile(InFilename));
	NewAsset->AssetImportData->SourceData = MoveTemp(Info);

	const FString FileContents = ReadFileContents(InFilename);
	ImportWebAPI(NewAsset, InFilename, FileContents)
	.Next([&](bool bInSucceeded)
	{
		ensure(IsInGameThread());

		if(bInSucceeded)
		{
			ensure(!NewAsset->GetWebAPISchema()->APIName.IsEmpty());
		
			const FName DesiredAssetName = FName(UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(TEXT("WAPI_") + NewAsset->GetWebAPISchema()->APIName));
			NewAsset->Rename(*MakeUniqueObjectName(NewAsset->GetOuter(), UWebAPIDefinition::StaticClass(), DesiredAssetName).ToString());

			NewAsset->GetGeneratorSettings().SetNamespace(UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(NewAsset->GetWebAPISchema()->APIName));
			PostImportWebAPI(NewAsset);

			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NewAsset);
		}
	})
	.Wait();

	return NewAsset;
}

bool UWebAPIDefinitionFactory::CanReimport(UObject* InObj, TArray<FString>& OutFilenames)
{
	if(const UWebAPIDefinition* Asset = Cast<UWebAPIDefinition>(InObj))
	{
		if(const UAssetImportData* AssetImportData = Asset->AssetImportData)
		{
			const bool bIsValidFile = UWebAPIDefinitionFactory::IsValidFile(AssetImportData->GetFirstFilename());
			if (!bIsValidFile)
			{
				return false;
			}
			OutFilenames.Add(AssetImportData->GetFirstFilename());
		}
		else
		{
			OutFilenames.Add(TEXT(""));
		}
		
		return Algo::AnyOf(OutFilenames, [this](const FString& InFileName)
		{
			const FString FileContents = ReadFileContents(InFileName);
			return CanImportWebAPI(InFileName, FileContents);
		});
	}
	return false;
}

void UWebAPIDefinitionFactory::SetReimportPaths(UObject* InObj, const TArray<FString>& NewReimportPaths)
{
	UWebAPIDefinition* Asset = Cast<UWebAPIDefinition>(InObj);
	if (Asset && ensure(NewReimportPaths.Num() == 1))
	{
		if (UAssetImportData* AssetImportData = Asset->AssetImportData)
		{
			AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
		}
	}
}

EReimportResult::Type UWebAPIDefinitionFactory::Reimport(UObject* InObj)
{
	UWebAPIDefinition* Definition = Cast<UWebAPIDefinition>(InObj);
	if (!Definition)
	{
		return EReimportResult::Failed;
	}

	UAssetImportData* AssetImportData = Definition->AssetImportData;
	if(!AssetImportData)
	{
		AssetImportData = Definition->AssetImportData = NewObject<UAssetImportData>(Definition, TEXT("AssetImportData"));
	}
	
	// Get the re-import filename
	const FString ImportedFilename = AssetImportData->GetFirstFilename();
	const bool bIsValidFile = UWebAPIDefinitionFactory::IsValidFile(ImportedFilename);
	if (!bIsValidFile)
	{
		return EReimportResult::Failed;
	}
	
	if (!ImportedFilename.Len())
	{
		return EReimportResult::Failed;
	}
	
	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*ImportedFilename) == INDEX_NONE)
	{
		UE_LOGF(LogWebAPIEditor, Warning, "UWebAPIDefinitionFactory::Reimport(): Cannot reimport, source file cannot be found.");
		return EReimportResult::Failed;
	}

	// Clear log before Generate
	Definition->GetMessageLog()->ClearLog();

	Definition->GetWebAPISchema()->Clear();
	const FString FileContents = ReadFileContents(ImportedFilename);

	const TFuture<EReimportResult::Type> ImportResult = ImportWebAPI(Definition, ImportedFilename, FileContents)
	.Next([this, &Definition, &AssetImportData, &ImportedFilename](bool bInSuccessful)
	{
		EReimportResult::Type ReimportResult = EReimportResult::Succeeded;
		if(!bInSuccessful)
		{
			UE_LOGF(LogWebAPIEditor, Warning, "UWebAPIDefinitionFactory::Reimport(): Load failed.");
			ReimportResult = EReimportResult::Failed;
		}

		PostImportWebAPI(Definition);

		// Broadcast import event, make sure all modifications are done before this
		AsyncTask(ENamedThreads::GameThread, [Definition]
		{
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(Definition);
		});
		
		// Only return success if ImportWebAPI returns true
		AssetImportData->Update(ImportedFilename);
		return ReimportResult;
	});

	// Blocking, but can remove if this is made async in the future
	ImportResult.Wait();

	return ImportResult.Get();
}

FString UWebAPIDefinitionFactory::ReadFileContents(const FString& InFileName) const
{
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *InFileName))
	{
		UE_LOGF(LogWebAPIEditor, Error, "UWebAPIDefinitionFactory::ReadFileContents(): File contents failed to load.");
		return TEXT("");
	}

	return FileContents;
}

bool UWebAPIDefinitionFactory::IsValidFile(const FString& InFilename) const
{
	const FString FileExtension = FPaths::GetExtension(InFilename, /*bIncludeDot*/ false);
	return IsValidFileExtension(FileExtension);
}

#undef LOCTEXT_NAMESPACE
