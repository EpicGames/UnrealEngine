// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "WebAPIEditorUtilities.h"

BEGIN_DEFINE_SPEC(FWebAPIEditorUtilitiesSpec,
	TEXT("Plugins.WebAPI.Editor.Utilities"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

END_DEFINE_SPEC(FWebAPIEditorUtilitiesSpec)

void FWebAPIEditorUtilitiesSpec::Define()
{
	using namespace UE::WebAPI;
	
	Describe(TEXT("ToPascalCase"), [this]
	{
		It(TEXT("Strips leading illegal tokens"), [this]
		{
			const FStringView TestString = TEXT("$teststring");
			const FString ExpectedString = TEXT("teststring");

			const FString ResultString = UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(TestString);

			// String equality is case-sensitive by default
			TestEqual("String doesn't contain illegal characters", ResultString, ExpectedString);
		});
	});
}

#endif
