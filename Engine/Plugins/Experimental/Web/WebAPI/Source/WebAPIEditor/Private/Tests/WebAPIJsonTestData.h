// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariantMeta.h"
#include "WebAPIJsonUtilities.h"

#include "WebAPIJsonTestData.generated.h"

UENUM()
enum class EWAPITestUEnum : uint8
{
	EnumValueA = 4,
	EnumValueB = 2,
	EnumValueC = 7
};

namespace UE::WebAPI::Testing
{
	enum class EWAPITestEnum : uint8
	{
		EnumValueA = 4,
		EnumValueB = 2,
		EnumValueC = 7
	};
	
	static TMap<FString, EWAPITestEnum> StringToTestEnum = {
		{ TEXT("EnumValueA"), EWAPITestEnum::EnumValueA },
		{ TEXT("EnumValueB"), EWAPITestEnum::EnumValueB },
		{ TEXT("EnumValueC"), EWAPITestEnum::EnumValueC },
	};

	struct WEBAPIEDITOR_API FTestStruct
	{
		float TestFloat = 45.2f;
		FText TestText = FText::FromString(TEXT("Test Text"));
		TSharedPtr<FTestStruct> TestPtrOfContainingType;
		TArray<TSharedPtr<FTestStruct>> ArrayOfOuterType;

		bool FromJson(const TSharedPtr<FJsonObject>& InJsonObject)
		{
			// at least one must deserialize
			bool bResult = false;
			bResult |= Json::TryGetField(InJsonObject, GET_MEMBER_NAME_STRING_CHECKED(FTestStruct, TestFloat), TestFloat);
			bResult |= Json::TryGetField(InJsonObject, GET_MEMBER_NAME_STRING_CHECKED(FTestStruct, TestText), TestText);
			bResult |= Json::TryGetField(InJsonObject, GET_MEMBER_NAME_STRING_CHECKED(FTestStruct, TestPtrOfContainingType), TestPtrOfContainingType);
			bResult |= Json::TryGetField(InJsonObject, GET_MEMBER_NAME_STRING_CHECKED(FTestStruct, ArrayOfOuterType), ArrayOfOuterType);
			return bResult;
		}
	};
	
	class WEBAPIEDITOR_API FTestClass
	{
	public:
		TArray<FString> ArrayOfStrings;
		TArray<FTestStruct> ArrayOfStructs;
		TArray<TSharedPtr<FTestStruct>> ArrayOfSharedPtr;
		TArray<TOptional<FTestStruct>> ArrayOfOptional;
		TArray<TSharedPtr<TOptional<FTestStruct>>> ArrayOfSharedPtrOptional;

		EWAPITestUEnum UEnumValue = EWAPITestUEnum::EnumValueA;
		EWAPITestEnum EnumValue = EWAPITestEnum::EnumValueA;
		TSharedPtr<EWAPITestUEnum> EnumSharedPtr;

		Json::TJsonReference<FTestStruct> JsonReferenceStruct;
		Json::TJsonReference<TSharedPtr<FTestStruct>> JsonReferenceStructSharedPtr;

		TMap<EWAPITestUEnum, TArray<float>> MapOfEnumToFloats;

		TVariant<EWAPITestUEnum, FTestStruct, bool> Variant;
	};
}
