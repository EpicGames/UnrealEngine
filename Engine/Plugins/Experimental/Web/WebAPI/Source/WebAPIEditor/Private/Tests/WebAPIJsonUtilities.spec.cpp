// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "Serialization/JsonSerializer.h"
#include "Tests/WebAPIJsonTestData.h"

BEGIN_DEFINE_SPEC(FWebAPIEditorJsonSpec,
	TEXT("Plugins.WebAPI.Editor.Json"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

	TSharedPtr<FJsonObject> JsonObject;

	// Deserialize to JsonObject
	bool DeserializeJsonString(const FString& InJsonString)
	{
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(InJsonString);
		return FJsonSerializer::Deserialize(JsonReader, JsonObject);
	}

END_DEFINE_SPEC(FWebAPIEditorJsonSpec)

void FWebAPIEditorJsonSpec::Define()
{
	using namespace UE::WebAPI;
		
	Describe(TEXT("Deserialization"), [this]
	{
		Describe(TEXT("Enum"), [this]
		{
			It(TEXT("loads from string"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "enumValue": "enumValueB"
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, EnumValue),
						TestObject.EnumValue,
						Testing::StringToTestEnum);
				
					TestTrue(TEXT("TryGetField"), bGotField);
					TestEqual(TEXT("EnumValue is EnumValueB"), TestObject.EnumValue, Testing::EWAPITestEnum::EnumValueB);
				}
			});

			// UEnum doesn't require a String to Value lookup - it gets that from StaticEnum<>
			It(TEXT("loads UEnum from string"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "uenumValue": "enumValueC"
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, UEnumValue),
						TestObject.UEnumValue);
				
					TestTrue(TEXT("TryGetField"), bGotField);
					TestEqual(TEXT("EnumValue is EnumValueC"), TestObject.UEnumValue, EWAPITestUEnum::EnumValueC);
				}
			});

			It(TEXT("doesn't load from string that doesn't name a valid value"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "enumValue": "invalidEnumValue"
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);
	
				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, EnumValue),
						TestObject.EnumValue,
						Testing::StringToTestEnum);
								
					TestFalse(TEXT("TryGetField"), bGotField);
				}
			});

			It(TEXT("loads from integer"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "enumValue": 7
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, EnumValue),
						TestObject.EnumValue,
						Testing::StringToTestEnum);
												
					TestTrue(TEXT("TryGetField"), bGotField);
					TestEqual(TEXT("EnumValue is EnumValueC"), TestObject.EnumValue, Testing::EWAPITestEnum::EnumValueC);
				}
			});
		});

		Describe(TEXT("Arrays"), [this]
		{
			Describe(TEXT("String"), [this]
			{
				It(TEXT("loads 2 strings"), [this]
				{
					static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
					{
						"arrayOfStrings": [
					        "test string",
					        "string test"
					    ]
					})_JSON");
					const bool bDeserialized = DeserializeJsonString(ExampleJson);
					TestTrue(TEXT("Deserialized Json"), bDeserialized);

					if(bDeserialized)
					{
						Testing::FTestClass TestObject;
						const bool bGotField = UE::Json::TryGetField(
							JsonObject,
							GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, ArrayOfStrings),
							TestObject.ArrayOfStrings);
								
						TestTrue(TEXT("TryGetField"), bGotField);
						TestEqual(TEXT("Array has 2 elements"), TestObject.ArrayOfStrings.Num(), 2);
						TestEqual(TEXT("Array[0] is test string"), TestObject.ArrayOfStrings[0], TEXT("test string"));
						TestEqual(TEXT("Array[1] is string test"), TestObject.ArrayOfStrings[1], TEXT("string test")); 
					}
				});
			});

			Describe(TEXT("Value"), [this]
			{
				It(TEXT("loads 3 elements"), [this]
				{
					static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
					{
					    "arrayOfStructs": [
					        {
					            "testFloat": 35.5            
					        },
					        {
					            "testText": "Some other text"         
					        },
					        {
					            "testPtrOfContainingType": {
					                "testFloat": 23.1            
					            }
					        }
					    ]
					})_JSON");
					const bool bDeserialized = DeserializeJsonString(ExampleJson);
					TestTrue(TEXT("Deserialized Json"), bDeserialized);

					if(bDeserialized)
					{
						Testing::FTestClass TestObject;
						const bool bGotField = UE::Json::TryGetField(
							JsonObject,
							GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, ArrayOfStructs),
							TestObject.ArrayOfStructs);
											
						TestTrue(TEXT("TryGetField"), bGotField);
						TestEqual(TEXT("Array has 3 elements"), TestObject.ArrayOfStructs.Num(), 3);
						TestEqual(TEXT("Array[0].testFloat is 35.5"), TestObject.ArrayOfStructs[0].TestFloat, 35.5f);
						TestEqual(TEXT("Array[1].testText is Some other text"), TestObject.ArrayOfStructs[1].TestText.ToString(), TEXT("Some other text"));
						TestEqual(TEXT("Array[2].testPtrOfContainingType.testFloat is 23.1"), TestObject.ArrayOfStructs[2].TestPtrOfContainingType->TestFloat, 23.1f);
					}
				});
			});

			Describe(TEXT("SharedPtr"), [this]
			{
				It(TEXT("loads 3 elements"), [this]
				{
					static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
					{
					    "arrayOfSharedPtr": [
					        {
					            "testFloat": 5.5            
					        },
					        {
					            "testText": "SharedPtr text"         
					        },
					        null
					    ]
					})_JSON");
					const bool bDeserialized = DeserializeJsonString(ExampleJson);
					TestTrue(TEXT("Deserialized Json"), bDeserialized);

					if(bDeserialized)
					{
						Testing::FTestClass TestObject;
						const bool bGotField = UE::Json::TryGetField(
							JsonObject,
							GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, ArrayOfSharedPtr),
							TestObject.ArrayOfSharedPtr);
	
						TestTrue(TEXT("TryGetField"), bGotField);
						TestEqual(TEXT("Array has 3 elements"), TestObject.ArrayOfSharedPtr.Num(), 3);
						TestEqual(TEXT("Array[0].testFloat"), TestObject.ArrayOfSharedPtr[0]->TestFloat, 5.5f);
						TestEqual(TEXT("Array[1].testText is SharedPtr text"), TestObject.ArrayOfSharedPtr[1]->TestText.ToString(), TEXT("SharedPtr text"));
						TestNull(TEXT("Array[2]"), TestObject.ArrayOfSharedPtr[2].Get());
					}
				});

				It(TEXT("loads 4 Optional elements"), [this]
				{
					static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
					{
					    "arrayOfSharedPtrOptional": [
							null,
					        {
					            "testFloat": 5.5            
					        },
					        {
					            "testText": "SharedPtr Optional text"         
					        },
					        null
					    ]
					})_JSON");
					const bool bDeserialized = DeserializeJsonString(ExampleJson);
					TestTrue(TEXT("Deserialized Json"), bDeserialized);

					if(bDeserialized)
					{
						Testing::FTestClass TestObject;					
						const bool bGotField = UE::Json::TryGetField(
							JsonObject,
							GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, ArrayOfSharedPtrOptional),
							TestObject.ArrayOfSharedPtrOptional);
							
						TestTrue(TEXT("TryGetField"), bGotField);
						TestEqual(TEXT("Array has 4 elements"), TestObject.ArrayOfSharedPtrOptional.Num(), 4);
						TestNull(TEXT("Array[0]"), TestObject.ArrayOfSharedPtrOptional[0].Get());
						TestEqual(TEXT("Array[1].testFloat"), TestObject.ArrayOfSharedPtrOptional[1]->GetValue().TestFloat, 5.5f);
						TestEqual(TEXT("Array[2].testText is SharedPtr Optional text"), TestObject.ArrayOfSharedPtrOptional[2]->GetValue().TestText.ToString(), TEXT("SharedPtr Optional text"));
						TestNull(TEXT("Array[3]"), TestObject.ArrayOfSharedPtrOptional[3].Get());
					}
				});
			});
		});

		Describe(TEXT("Variant"), [this]
		{
			It(TEXT("loads enum"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "Variant": "enumValueC"
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, Variant),
						TestObject.Variant);

					if(TestTrue(TEXT("TryGetField"), bGotField))
					{
						TestTrue(TEXT("Variant is an enum"), TestObject.Variant.IsType<EWAPITestUEnum>());
						TestEqual(TEXT("Enum value is EnumValueC"), TestObject.Variant.Get<EWAPITestUEnum>(), EWAPITestUEnum::EnumValueC);
					}
				}
			});

			It(TEXT("loads struct"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "Variant": {
						"testFloat": 16.5,
						"testText": "Variant nested text"
					}
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, Variant),
						TestObject.Variant);

					if(TestTrue(TEXT("TryGetField"), bGotField))
					{
						TestTrue(TEXT("Variant is a struct"), TestObject.Variant.IsType<Testing::FTestStruct>());
						TestEqual(TEXT("Variant.testFloat is 16.5"), TestObject.Variant.Get<Testing::FTestStruct>().TestFloat, 16.5f);
						TestEqual(TEXT("Variant.testText is Variant nested text"), TestObject.Variant.Get<Testing::FTestStruct>().TestText.ToString(), TEXT("Variant nested text"));
					}
				}
			});

			It(TEXT("loads boolean"), [this]
			{
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "Variant": false
				})_JSON");
				const bool bDeserialized = DeserializeJsonString(ExampleJson);
				TestTrue(TEXT("Deserialized Json"), bDeserialized);

				if(bDeserialized)
				{
					Testing::FTestClass TestObject;
					const bool bGotField = UE::Json::TryGetField(
						JsonObject,
						GET_MEMBER_NAME_STRING_CHECKED(Testing::FTestClass, Variant),
						TestObject.Variant);

					if(TestTrue(TEXT("TryGetField"), bGotField))
					{
						TestEqual(TEXT("Field is a boolean"), JsonObject->GetField<EJson::Boolean>(TEXT("Variant"))->Type, EJson::Boolean);
						TestTrue(TEXT("Variant is a boolean"), TestObject.Variant.IsType<bool>());
						TestFalse(TEXT("Bool value is false"), TestObject.Variant.Get<bool>());
					}
				}
			});
		});
	});
}

#endif
