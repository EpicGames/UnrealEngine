// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIDefinition.h"

#include "GameProjectUtils.h"
#include "ModuleDescriptor.h"
#include "ScopedTransaction.h"
#include "WebAPIEditorLog.h"
#include "Dom/WebAPISchema.h"
#include "Dom/WebAPITypeRegistry.h"
#include "Misc/Paths.h"
#include "Templates/SubclassOf.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif

#define LOCTEXT_NAMESPACE "WebAPIDefinition"

FString FWebAPIDefinitionTargetModule::GetPath()
{
	// Path already fetched, verified and cached.
	if(!ResolvedPath.IsEmpty() && FPaths::DirectoryExists(ResolvedPath))
	{
		return ResolvedPath;
	}

	if(!AbsolutePath.IsEmpty()
		&& FPaths::DirectoryExists(AbsolutePath))
	{
		ResolvedPath = AbsolutePath;
		return ResolvedPath;
	}

	TArray<FModuleContextInfo> CurrentModules = GameProjectUtils::GetCurrentProjectModules();
	CurrentModules.Append(GameProjectUtils::GetCurrentProjectPluginModules());
	check(!CurrentModules.IsEmpty());

	const FModuleContextInfo* FoundModule = CurrentModules.FindByPredicate([&](const FModuleContextInfo& InModuleInfo)
	{
		return InModuleInfo.ModuleName.Equals(Name, ESearchCase::IgnoreCase);
	});

	if(FoundModule)
	{
		ResolvedPath = FoundModule->ModuleSourcePath;
		return ResolvedPath;
	}

	UE_LOGF(LogWebAPIEditor, Error, "Module path for \"%ls\" could not be resolved.", *Name);
	return TEXT("");
}

UWebAPIDefinition::UWebAPIDefinition()
{
#if WITH_EDITORONLY_DATA
	AssetImportData = CreateEditorOnlyDefaultSubobject<UAssetImportData>(TEXT("AssetImportData"));
#endif

	WebAPISchema = CreateDefaultSubobject<UWebAPISchema>(TEXT("WebAPISchema"));
	MessageLog = MakeShared<FWebAPIMessageLog>();
	
	GeneratorSettings.OnNamespaceChanged().AddUObject(this, &UWebAPIDefinition::OnNamespaceChanged);
}

UWebAPIDefinition::~UWebAPIDefinition()
{
	GeneratorSettings.OnNamespaceChanged().RemoveAll(this);
}

UObject* UWebAPIDefinition::AddOrGetImportedDataCache(FName InKey, const TSubclassOf<UObject>& InDataCacheClass)
{
	TObjectPtr<UObject>* Result = ImportedDataCache.Find(InKey);
	if(Result && *Result)
	{
		return *Result;
	}

	FScopedTransaction Transaction(LOCTEXT("WebAPIDefinition_AddImportedDataCache", "Add Imported Data Cache"));
	Modify();
	return ImportedDataCache.Add(InKey, NewObject<UObject>(this, InDataCacheClass));
}

FString UWebAPIDefinition::GetCopyrightNotice() const
{
	return GeneratorSettings.CopyrightNotice;
}

const FWebAPIProviderSettings& UWebAPIDefinition::GetProviderSettings() const
{
	return ProviderSettings;
}

FWebAPIProviderSettings& UWebAPIDefinition::GetProviderSettings()
{
	return ProviderSettings;
}

const FWebAPICodeGeneratorSettings& UWebAPIDefinition::GetGeneratorSettings() const
{
	return GeneratorSettings;
}

FWebAPICodeGeneratorSettings& UWebAPIDefinition::GetGeneratorSettings()
{
	return GeneratorSettings;
}

UWebAPISchema* UWebAPIDefinition::GetWebAPISchema()
{
	if(ensure(WebAPISchema))
	{
		return WebAPISchema;
	}

	return nullptr;
}

const UWebAPISchema* UWebAPIDefinition::GetWebAPISchema() const
{
	if(ensure(WebAPISchema))
	{
		return WebAPISchema;
	}

	return nullptr;
}

const TSharedPtr<FWebAPIMessageLog>& UWebAPIDefinition::GetMessageLog() const
{
	return MessageLog;
}

EDataValidationResult UWebAPIDefinition::IsDataValid(FDataValidationContext& Context) const
{
	return CombineDataValidationResults(
		GetWebAPISchema()->IsDataValid(Context),
		GetWebAPISchema()->TypeRegistry->IsDataValid(Context));
}

void UWebAPIDefinition::OnNamespaceChanged(const FString& InNewNamespace)
{
	GetWebAPISchema()->SetNamespace(InNewNamespace);
}

#undef LOCTEXT_NAMESPACE
