// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorCommands.h"

#define LOCTEXT_NAMESPACE "WebAPIEditorCommands"

void FWebAPIEditorCommands::RegisterCommands()
{
	UI_COMMAND(Generate, "Generate", "Generate files for the current Web API", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
