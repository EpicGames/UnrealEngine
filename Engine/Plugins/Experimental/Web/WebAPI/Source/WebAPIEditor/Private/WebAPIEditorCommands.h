// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FWebAPIEditorCommands
	: public TCommands<FWebAPIEditorCommands>
{
public:
	FWebAPIEditorCommands()
		: TCommands<FWebAPIEditorCommands>(TEXT("WebAPIEditor"),
			NSLOCTEXT("Contexts", "WebAPIEditor", "WebAPI Definition Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{ }

	/** Generate command to execute code generation. */
	TSharedPtr<FUICommandInfo> Generate;

	/** Registers all associated commands. */
	virtual void RegisterCommands() override;
};
