// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorModule.h"

#include "CodeGen/WebAPICodeGenerator.h"
#include "IPluginBrowser.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "WebAPIDefinition.h"
#include "WebAPIEditorCommands.h"
#include "WebAPIEditorLog.h"
#include "WebAPIEditorSettings.h"
#include "WebAPIEditorStyle.h"
#include "WebAPIPluginWizardDefinition.h"
#include "Assets/WebAPIDefinitionAssetTypeActions.h"
#include "Details/WebAPIDefinitionDetailsCustomization.h"
#include "Details/WebAPIDefinitionTargetModuleCustomization.h"
#include "Dom/WebAPITypeRegistry.h"
#include "WebAPIProvider.h"

#define LOCTEXT_NAMESPACE "WebAPIEditor"

const FName FWebAPIEditorModule::AppIdentifier(TEXT("WebAPIEditorApp"));

const FName FWebAPIEditorModule::PluginCreatorTabName(TEXT("WebAPIPluginCreator"));

void FWebAPIEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
	ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	Actions.Emplace(MakeShared<FWebAPIDefinitionAssetTypeActions>());
	AssetTools.RegisterAssetTypeActions(Actions.Last().ToSharedRef());

	AssetCategory = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("WebAPI")), NSLOCTEXT("WebAPI", "WebAPI", "Web API"));

	// Disable any UI feature if running in command mode
	if (UToolMenus::IsToolMenuUIEnabled())
	{
		FWebAPIEditorStyle::Register();
		FWebAPIEditorCommands::Register();

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout(UWebAPIDefinition::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FWebAPIDefinitionDetailsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FWebAPIDefinitionTargetModule::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWebAPIDefinitionTargetModuleCustomization::MakeInstance));

		// Register nomad tab for new plugin/module creation
		{
			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
				PluginCreatorTabName,
				FOnSpawnTab::CreateLambda([](const FSpawnTabArgs& SpawnTabArgs)
				{
					const TSharedRef<FWebAPIPluginWizardDefinition> PluginWizardDefinition = MakeShared<FWebAPIPluginWizardDefinition>();
					return IPluginBrowser::Get().SpawnPluginCreatorTab(SpawnTabArgs, PluginWizardDefinition);
				}))
				.SetDisplayName(LOCTEXT("NewPluginTabHeader", "New Plugin"))
				.SetMenuType(ETabSpawnerMenuType::Hidden);

			const FVector2D DefaultSize(1000.0f, 750.0f);
			FTabManager::RegisterDefaultTabWindowSize(PluginCreatorTabName, DefaultSize);
		}
	}
}

void FWebAPIEditorModule::ShutdownModule()
{
	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	if (const FAssetToolsModule* ModuleInterface = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		IAssetTools& AssetTools = ModuleInterface->Get();
		for (TSharedPtr<IAssetTypeActions>& Action : Actions)
		{
			if (Action.IsValid())
			{
				AssetTools.UnregisterAssetTypeActions(Action.ToSharedRef());
			}
		}
	}

	// Disable any UI feature if running in command mode
	if (UToolMenus::IsToolMenuUIEnabled())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( PluginCreatorTabName );
		
		FWebAPIEditorCommands::Unregister();
		FWebAPIEditorStyle::Unregister();

		// Register the details customizer
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyEditorModule.UnregisterCustomClassLayout(TEXT("WebAPIDefinition"));
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(TEXT("WebAPIDefinitionTargetModule"));
	}
}

EAssetTypeCategories::Type FWebAPIEditorModule::GetAssetCategory() const
{
	return AssetCategory;
}

bool FWebAPIEditorModule::AddProvider(FName InProviderName, TSharedRef<IWebAPIProviderInterface> InProvider)
{
	if(Providers.Contains(InProviderName))
	{
		UE_LOGF(LogWebAPIEditor, Warning, "FWebAPIEditorModule::AddProvider(): Provider \"%ls\" is already registered, skipping.", *InProviderName.ToString());
		return false;
	}
	
	Providers.Emplace(InProviderName, InProvider);
	ProvidersChangedDelegate.Broadcast();
	
	return true;
}

void FWebAPIEditorModule::RemoveProvider(FName InProviderName)
{
	if(!Providers.Contains(InProviderName))
	{
		UE_LOGF(LogWebAPIEditor, Warning, "FWebAPIEditorModule::RemoveProvider(): Provider \"%ls\" wasn't found for removal, skipping.", *InProviderName.ToString());
		return;
	}

	Providers.Remove(InProviderName);
	ProvidersChangedDelegate.Broadcast();
}

TSharedPtr<IWebAPIProviderInterface> FWebAPIEditorModule::GetProvider(FName InProviderName)
{
	if(const TSharedRef<IWebAPIProviderInterface>* FoundProvider = Providers.Find(InProviderName))
	{
		return *FoundProvider;
	}

	UE_LOGF(LogWebAPIEditor, Error, "FWebAPIEditorModule::GetProvider(): Provider \"%ls\" wasn't found!", *InProviderName.ToString());
	return nullptr;
}

bool FWebAPIEditorModule::GetProviders(TArray<TSharedRef<IWebAPIProviderInterface>>& OutProviders)
{
	Providers.GenerateValueArray(OutProviders);
	return !OutProviders.IsEmpty();
}

bool FWebAPIEditorModule::AddCodeGenerator(FName InCodeGeneratorName, const TSubclassOf<UObject>& InCodeGenerator)
{
	if(CodeGenerators.Contains(InCodeGeneratorName))
	{
		UE_LOGF(LogWebAPIEditor, Warning, "FWebAPIEditorModule::AddCodeGenerator(): CodeGenerator \"%ls\" is already registered, skipping.", *InCodeGeneratorName.ToString());
		return false;
	}

	check(InCodeGenerator);
	check(InCodeGenerator->ImplementsInterface(UWebAPICodeGeneratorInterface::StaticClass()));

	CodeGenerators.Emplace(InCodeGeneratorName, InCodeGenerator.GetDefaultObject());

	// Ensure settings has a code generator specified
	UWebAPIEditorSettings* ProjectSettings = GetMutableDefault<UWebAPIEditorSettings>();
	if(ProjectSettings->CodeGeneratorClass.IsNull())
	{
		ProjectSettings->CodeGeneratorClass = InCodeGenerator;
	}

	return true;
}

void FWebAPIEditorModule::RemoveCodeGenerator(FName InCodeGeneratorName)
{
	if(!CodeGenerators.Contains(InCodeGeneratorName))
	{
		UE_LOGF(LogWebAPIEditor, Warning, "FWebAPIEditorModule::RemoveCodeGenerator(): CodeGenerator \"%ls\" wasn't found for removal, skipping.", *InCodeGeneratorName.ToString());
		return;
	}

	CodeGenerators.Remove(InCodeGeneratorName);
}

TScriptInterface<IWebAPICodeGeneratorInterface> FWebAPIEditorModule::GetCodeGenerator(FName InCodeGeneratorName)
{
	if(const TScriptInterface<IWebAPICodeGeneratorInterface>* FoundGenerator = CodeGenerators.Find(InCodeGeneratorName))
	{
		return *FoundGenerator;
	}

	UE_LOGF(LogWebAPIEditor, Error, "FWebAPIEditorModule::GetCodeGenerator(): CodeGenerator \"%ls\" wasn't found!", *InCodeGeneratorName.ToString());
	return {};
}

bool FWebAPIEditorModule::GetCodeGenerators(TArray<TScriptInterface<IWebAPICodeGeneratorInterface>>& OutCodeGenerators)
{
	CodeGenerators.GenerateValueArray(OutCodeGenerators);
	return !OutCodeGenerators.IsEmpty();
}

UWebAPIStaticTypeRegistry* FWebAPIEditorModule::GetStaticTypeRegistry() const
{
	return GEngine ? GEngine->GetEngineSubsystem<UWebAPIStaticTypeRegistry>() : nullptr;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebAPIEditorModule, WebAPIEditor)
