// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebAPIEditorModule.h"


class FExtensibilityManager;

class IWebAPIProviderInterface;
class FWebAPIDefinitionAssetEditorToolkit;

class FWebAPIEditorModule final
    : public IWebAPIEditorModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;

    virtual EAssetTypeCategories::Type GetAssetCategory() const override;

    //~ Begin IHasMenuExtensibilityinterface
    virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
    //~ End IHasMenuExtensibilityinterface

    //~ Begin IHasToolBarExtensibility
    virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
    //~ End IHasToolBarExtensibility

    /** Adds and registers a WebAPI provider. */
    virtual bool AddProvider(FName InProviderName, TSharedRef<IWebAPIProviderInterface> InProvider) override;
    
    /** Removes a WebAPI provider. */
    virtual void RemoveProvider(FName InProviderName) override;

	/** Retrieve a registered provider by its name. */
	virtual TSharedPtr<IWebAPIProviderInterface> GetProvider(FName InProviderName) override;
    
    /** Retrieve all registered providers. */
    virtual bool GetProviders(TArray<TSharedRef<IWebAPIProviderInterface>>& OutProviders) override;

    /** Whenever a Provider is added or removed. */
    virtual FOnWebAPIProvidersChanged& OnProvidersChanged() override { return ProvidersChangedDelegate; }

    /** Adds and registers a WebAPI code generator. */
    virtual bool AddCodeGenerator(FName InCodeGeneratorName, const TSubclassOf<UObject>& InCodeGenerator) override;

    /** Removes a WebAPI code generator. */
    virtual void RemoveCodeGenerator(FName InCodeGeneratorName) override;

	/** Retrieve a registered code generator by its name. */
	virtual TScriptInterface<IWebAPICodeGeneratorInterface> GetCodeGenerator(FName InCodeGeneratorName) override;

    /** Retrieve all registered code generators. */
    virtual bool GetCodeGenerators(TArray<TScriptInterface<IWebAPICodeGeneratorInterface>>& OutCodeGenerators) override;

    /** Returns the static type registry to retrieve metadata for built-in types. */
    virtual UWebAPIStaticTypeRegistry* GetStaticTypeRegistry() const override;

	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() override { return RegisterLayoutExtensions; }

public:
    /** App identifier string */
    static const FName AppIdentifier;

	/** ID name for the plugin creator tab */
	static const FName PluginCreatorTabName;

private:
    EAssetTypeCategories::Type AssetCategory = EAssetTypeCategories::Misc;
    TWeakPtr<FWebAPIDefinitionAssetEditorToolkit> AssetEditorToolkit;
    TArray<TSharedPtr<class IAssetTypeActions>> Actions;

    TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
    TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
    
    TMap<FName, TSharedRef<IWebAPIProviderInterface>> Providers;
    FOnWebAPIProvidersChanged ProvidersChangedDelegate;

    TMap<FName, TScriptInterface<IWebAPICodeGeneratorInterface>> CodeGenerators;

	FOnRegisterLayoutExtensions	RegisterLayoutExtensions;
};
