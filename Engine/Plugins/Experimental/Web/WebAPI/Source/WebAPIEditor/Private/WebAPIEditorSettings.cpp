// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorSettings.h"

#include "CodeGen/WebAPICodeGenerator.h"

TScriptInterface<IWebAPICodeGeneratorInterface> UWebAPIEditorSettings::GetGeneratorClass() const
{
	if(!CodeGeneratorClass.IsNull())
	{
		const TScriptInterface<IWebAPICodeGeneratorInterface> CodeGeneratorInstance = CodeGeneratorClass->GetDefaultObject();
		return CodeGeneratorInstance;		
	}

	return nullptr;
}
