// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("WebAPIEditor"), STATGROUP_WebAPIEditor, STATCAT_Advanced);
