// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir UE::WebAPI::Private::StyleInstance->RootToContentDir
#define RootToCoreContentDir UE::WebAPI::Private::StyleInstance->RootToCoreContentDir

namespace UE
{
	namespace WebAPI
	{ 
		namespace Private
		{
			const FVector2D Icon16x16(16.0f, 16.0f);
			const FVector2D Icon20x20(20.0f, 20.0f);
			const FVector2D Icon40x40(40.0f, 40.0f);
			const FVector2D Icon64x64(64.0f, 64.0f);

			const FName NAME_StyleName(TEXT("WebAPIEditorStyle"));

			static TUniquePtr<FSlateStyleSet> StyleInstance;	
		}
	}
}

TUniquePtr<FSlateStyleSet> FWebAPIEditorStyle::StyleInstance = nullptr;

void FWebAPIEditorStyle::Register()
{
	UE::WebAPI::Private::StyleInstance = MakeUnique<FSlateStyleSet>(UE::WebAPI::Private::NAME_StyleName);
	UE::WebAPI::Private::StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	UE::WebAPI::Private::StyleInstance->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("WebAPI"))->GetContentDir() / TEXT("Editor/Slate"));

	UE::WebAPI::Private::StyleInstance->Set("WebAPI.TreeView.LabelBackground", new CORE_BOX_BRUSH("Common/GroupBorderLight", 6.f/18.f, FStyleColors::Dropdown));
	
	FSlateStyleRegistry::RegisterSlateStyle(*UE::WebAPI::Private::StyleInstance.Get());
}

void FWebAPIEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*UE::WebAPI::Private::StyleInstance.Get());
	UE::WebAPI::Private::StyleInstance.Reset();
}

FName FWebAPIEditorStyle::GetStyleSetName()
{
	return UE::WebAPI::Private::NAME_StyleName;
}

const ISlateStyle& FWebAPIEditorStyle::Get()
{
	check(UE::WebAPI::Private::StyleInstance.IsValid());
	return *UE::WebAPI::Private::StyleInstance.Get();
}

#undef RootToContentDir
#undef RootToCoreContentDir
