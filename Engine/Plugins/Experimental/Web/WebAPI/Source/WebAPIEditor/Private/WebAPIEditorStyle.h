// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

class ISlateStyle;

/** Manages the style which provides resources for WebAPI Editor widgets. */
class FWebAPIEditorStyle
{
public:
	static void Register();
	static void Unregister();

	static FName GetStyleSetName();
	
	static const ISlateStyle& Get();

private:
	static TUniquePtr<class FSlateStyleSet> Create();
	static TUniquePtr<class FSlateStyleSet> StyleInstance;
};
