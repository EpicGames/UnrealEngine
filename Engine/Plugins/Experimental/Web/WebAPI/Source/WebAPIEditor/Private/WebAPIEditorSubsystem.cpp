// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorSubsystem.h"

#include "IWebAPIEditorModule.h"
#include "WebAPIDefinition.h"
#include "WebAPIEditorLog.h"
#include "WebAPIProvider.h"

UWebAPIEditorSubsystem::UWebAPIEditorSubsystem()
{
}

void UWebAPIEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	OnDefinitionPreImport().AddUObject(this, &UWebAPIEditorSubsystem::HandleDefinitionPreImport);
	OnDefinitionPostImport().AddUObject(this, &UWebAPIEditorSubsystem::HandleDefinitionPostImport);
}

void UWebAPIEditorSubsystem::Deinitialize()
{
	OnDefinitionPreImport().RemoveAll(this);
	OnDefinitionPostImport().RemoveAll(this);
	
	Super::Deinitialize();
}

UWebAPIEditorSubsystem::FOnWebAPIDefinitionPreImport& UWebAPIEditorSubsystem::OnDefinitionPreImport()
{ 
	return OnDefinitionPreImportDelegate;
}

UWebAPIEditorSubsystem::FOnWebAPIDefinitionPostImport& UWebAPIEditorSubsystem::OnDefinitionPostImport()
{
	return OnDefinitionPostImportDelegate;
}

UWebAPIEditorSubsystem::FOnModuleCreated& UWebAPIEditorSubsystem::OnModuleCreated()
{
	return OnModuleCreatedDelegate;
}

void UWebAPIEditorSubsystem::HandleDefinitionPreImport(UWebAPIDefinition* InDefinition)
{
}

void UWebAPIEditorSubsystem::HandleDefinitionPostImport(bool bInImportSuccessful, FName InFactoryName, UWebAPIDefinition* InDefinition)
{
	if(!bInImportSuccessful)
	{
		return;
	}

	checkf(InFactoryName != NAME_None, TEXT("The importing Factory must provide a name."));
	
	if(const TSharedPtr<IWebAPIProviderInterface> NamedProvider = IWebAPIEditorModuleInterface::Get().GetProvider(InFactoryName))
	{
		NamedProvider->ConvertToWebAPISchema(InDefinition)
		.Next([](EWebAPIConversionResult bInConversionResult)
		{
			
		});
		return;
	}

	UE_LOGF(LogWebAPIEditor, Error, "UWebAPIEditorSubsystem::HandleDefinitionPostImport(): Provider \"%ls\" wasn't found!", *InFactoryName.ToString());
}
