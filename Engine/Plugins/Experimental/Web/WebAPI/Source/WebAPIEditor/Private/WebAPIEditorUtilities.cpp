// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIEditorUtilities.h"

#include "Misc/DateTime.h"
#include "SourceCodeNavigation.h"
#include "Internationalization/BreakIterator.h"
#include "Templates/SubclassOf.h"

namespace UE
{
	namespace WebAPI
	{
		TSharedRef<FWebAPIStringUtilities> FWebAPIStringUtilities::Get()
		{
			static TSharedRef<FWebAPIStringUtilities> Instance = MakeShared<FWebAPIStringUtilities>();
			return Instance;
		}

		FString FWebAPIStringUtilities::ToPascalCase(const FStringView InString)
		{
			check(!InString.IsEmpty());

			if(!BreakIterator.IsValid())
			{
				BreakIterator = FBreakIterator::CreateCamelCaseBreakIterator();
			}

			FString CleanString = FString(InString);
			
			// Remove apostrophes before converting case, to avoid "You're" becoming "YouRe"
			CleanString.ReplaceInline(TEXT("'"), TEXT(""));
			CleanString.ReplaceInline(TEXT("`"), TEXT(""));
			CleanString.ReplaceInline(TEXT("\""), TEXT(""));

			FString Result;
			BreakIterator->SetStringRef(CleanString);
			for (int32 PrevBreak = 0, NameBreak = BreakIterator->MoveToNext(); NameBreak != INDEX_NONE; NameBreak = BreakIterator->MoveToNext())
			{
				Result.AppendChar(FChar::ToUpper(CleanString[PrevBreak++]));
				if(PrevBreak < CleanString.Len())
				{
					Result.AppendChars(&CleanString[PrevBreak], NameBreak - PrevBreak);
				}
				PrevBreak = NameBreak;
			}
			BreakIterator->ClearString();
			BreakIterator->ResetToBeginning();

			// remove spaces
			Result.ReplaceInline(TEXT(" "), TEXT(""));

			// accounts for snake_case
			Result.ReplaceInline(TEXT("_"), TEXT(""));

			// Only replace these if the Name is NOT a datetime
			FDateTime ParsedDateTime;
			bool bDateTimeParsed = FDateTime::Parse(Result, ParsedDateTime);
			// Between 4 and 8 (+2 for delimeters) characters
			if(bDateTimeParsed || FMath::IsWithin(Result.Len(), 4, 8 + 2 + 1))
			{
				// Datetime Parse failed, but might still be date-time if 4-8 characters
				if(FChar::IsDigit(Result[0]))
				{
					static const TCHAR* Delimeters[4] = { TEXT("_"), TEXT("-"), TEXT("."), TEXT("/") };

					TArray<FString> SplitString;
					Result.ParseIntoArray(SplitString, Delimeters, 4);
					// date-time will have 2 or 3 components
					if(FMath::IsWithin(SplitString.Num(), 2, 3 + 1)) 
					{
						bDateTimeParsed = true;
					}
				}
			}
			
			if(!bDateTimeParsed)
			{
				// accounts for dashes
				Result.ReplaceInline(TEXT("-"), TEXT(""));
                
				// accounts for dots
				Result.ReplaceInline(TEXT("."), TEXT(""));
			}

			ensureAlwaysMsgf(!Result.IsEmpty(), TEXT("Resulting string is empty!"));

			return Result;
		}

		FString FWebAPIStringUtilities::ToInitials(FStringView InString)
		{
			check(!InString.IsEmpty());

			if(!BreakIterator.IsValid())
			{
				BreakIterator = FBreakIterator::CreateCamelCaseBreakIterator();
			}

			FString Result;
			BreakIterator->SetStringRef(InString);
			for (int32 PrevBreak = 0, NameBreak = BreakIterator->MoveToNext(); NameBreak != INDEX_NONE; NameBreak = BreakIterator->MoveToNext())
			{
				Result.AppendChar(FChar::ToUpper(InString[PrevBreak++]));
				PrevBreak = NameBreak;
			}
			BreakIterator->ClearString();
			BreakIterator->ResetToBeginning();

			return Result;
		}

		FString FWebAPIStringUtilities::MakeValidMemberName(FStringView InString, const FString& InPrefix) const
		{
			check(!InString.IsEmpty());
			
			FString Result(InString);
			
			// accounts for $, @, # (common keyword tokens, not needed here)
 			Result.ReplaceInline(TEXT("$"), TEXT(""));
			Result.ReplaceInline(TEXT("@"), TEXT(""));
			Result.ReplaceInline(TEXT("#"), TEXT(""));

			// remove brackets/braces
			Result.ReplaceInline(TEXT("{"), TEXT(""));
			Result.ReplaceInline(TEXT("}"), TEXT(""));
			Result.ReplaceInline(TEXT("("), TEXT(""));
			Result.ReplaceInline(TEXT(")"), TEXT(""));
			Result.ReplaceInline(TEXT("{"), TEXT(""));
			Result.ReplaceInline(TEXT("}"), TEXT(""));

			// replace wildcard * with "Any"
			Result.ReplaceInline(TEXT("*"), TEXT("Any"));

			// accounts for apostrophes
			Result.ReplaceInline(TEXT("'"), TEXT(""));
			Result.ReplaceInline(TEXT("`"), TEXT(""));
			Result.ReplaceInline(TEXT("\""), TEXT(""));

			// accounts for commas
			Result.ReplaceInline(TEXT(","), TEXT(""));

			// accounts for spaces
			Result.ReplaceInline(TEXT(" "), TEXT(""));

			Result.ReplaceInline(TEXT("_"), TEXT(""));

			// accounts for dashes
			Result.ReplaceInline(TEXT("-"), TEXT(""));
                
			// accounts for dots
			Result.ReplaceInline(TEXT("."), TEXT(""));

			// accounts for slash
			Result.ReplaceInline(TEXT("/"), TEXT("_"));

			Result.ReplaceInline(TEXT("+"), TEXT(""));
			Result.ReplaceInline(TEXT("*"), TEXT(""));

			ensureAlwaysMsgf(!Result.IsEmpty(), TEXT("Resulting string is empty!"));

			// if it starts with a number, prepend the initials followed by an underscore
			if(Result.Left(1).IsNumeric())
			{
				ensureAlwaysMsgf(!InPrefix.IsEmpty(), TEXT("The member name is invalid (is a number, etc.) but no prefix was provided."));				
				Result = InPrefix + TEXT("_") + Result;
			}

			return Result;
		}

		TSharedRef<FWebAPIEditorUtilities> FWebAPIEditorUtilities::Get()
		{
			static TSharedRef<FWebAPIEditorUtilities> Instance = MakeShared<FWebAPIEditorUtilities>();
			return Instance;
		}

		bool FWebAPIEditorUtilities::GetHeadersForClass(const TSubclassOf<UObject>& InClass, TArray<FString>& OutHeaders)
		{ 
			if(const FString* IncludePath = InClass->FindMetaData(TEXT("IncludePath")))
			{
				OutHeaders.Add(*IncludePath);
				return true;
			}
			return false;
		}

		bool FWebAPIEditorUtilities::GetModulesForClass(const TSubclassOf<UObject>& InClass, TArray<FString>& OutModules)
		{
			FString ModuleName;
			if(FSourceCodeNavigation::FindClassModuleName(InClass, ModuleName))
			{
				OutModules.Add(ModuleName);
				return true;
			}
			return false;
		}
	}
}	
 
