// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"

class FWebAPIEditorWidgetsUtilities
{
public:
	static FSlateColor GetColorForVerb(uint8 InVerb) { return FSlateColor(FColor{255, 128, 64}); }
	static FName GetIconNameForVerb(uint8 InVerb) { return NAME_None; }
};
