// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIMessageLog.h"

#include "IMessageLogListing.h"
#include "Internationalization/Internationalization.h"
#include "MessageLogModule.h"
#include "Misc/DateTime.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "WebAPIMessageLog"

FWebAPIMessageLog::FWebAPIMessageLog()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	FMessageLogInitializationOptions LogOptions;

	// Don't show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages   = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear  = true;
	LogOptions.MaxPageCount = 1;

	MessageLogListing = MessageLogModule.CreateLogListing("WebAPI", LogOptions);
}

void FWebAPIMessageLog::LogInfo(const FText& InMessage, const FString& InCallerName)
{
	Log<EVerbosityLevel::Log>(InMessage, InCallerName);
}

void FWebAPIMessageLog::LogWarning(const FText& InMessage, const FString& InCallerName)
{
	Log<EVerbosityLevel::Warning>(InMessage, InCallerName);
}

void FWebAPIMessageLog::LogError(const FText& InMessage, const FString& InCallerName)
{
	Log<EVerbosityLevel::Error>(InMessage, InCallerName);
}

template <EVerbosityLevel VerbosityLevel>
void FWebAPIMessageLog::Log(const FText& InMessage, const FString& InCallerName)
{
	if (!ensure(MessageLogListing.IsValid()))
	{
		return;
	}

	// Static conversion map from EVerbosityLevel to EMessageSeverity
	static TMap<EVerbosityLevel, EMessageSeverity::Type> VerbosityToSeverity = 
	{
		{ EVerbosityLevel::Log, EMessageSeverity::Info },
		{ EVerbosityLevel::Warning, EMessageSeverity::Warning },
		{ EVerbosityLevel::Error, EMessageSeverity::Error }
	};

	const EMessageSeverity::Type Severity = VerbosityToSeverity[VerbosityLevel];
	
	TArray<TSharedRef<FTokenizedMessage>> Messages;
	const TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(Severity);
	
	Line->AddToken(FTextToken::Create(FText::Format(LOCTEXT("TimeStampFormat", "{0}"), FText::FromString(FDateTime::Now().ToString(TEXT("%H:%M:%S - "))))));
	Line->AddToken(FTextToken::Create(FText::Format(LOCTEXT("CallerNameFormat", "{0} - "), FText::FromString(InCallerName))));
	Line->AddToken(FTextToken::Create(InMessage));
	Messages.Add(Line);

	// Only send to UE_LOG if Warning or Error
	constexpr bool bMirrorToOutputLog = VerbosityLevel > EVerbosityLevel::Warning;
	MessageLogListing->AddMessages(MoveTemp(Messages), bMirrorToOutputLog);

	// Always select last message, that keep the UI widget scrolling
	constexpr bool bSelected = true;
	MessageLogListing->SelectMessage(Line, bSelected);
}

void FWebAPIMessageLog::ClearLog() const
{
	MessageLogListing->ClearMessages();
}

#undef LOCTEXT_NAMESPACE
