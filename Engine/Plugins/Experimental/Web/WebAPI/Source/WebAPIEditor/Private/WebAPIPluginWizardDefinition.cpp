// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIPluginWizardDefinition.h"

#include "Editor.h"
#include "Features/IPluginsEditorFeature.h"
#include "WebAPIEditorSubsystem.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "PluginReferenceDescriptor.h"

#define LOCTEXT_NAMESPACE "WebAPIPluginWizardDefinition"

/** A plugin template description that applies template-specific logic. */
struct FWebAPIGeneratedPluginTemplateDescription : public FPluginTemplateDescription
{
	FWebAPIGeneratedPluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath, bool bInCanContainContent = true, EHostType::Type InModuleDescriptorType = EHostType::Runtime, ELoadingPhase::Type InLoadingPhase = ELoadingPhase::Default)
		: FPluginTemplateDescription(InName, InDescription, InOnDiskPath, bInCanContainContent, InModuleDescriptorType, InLoadingPhase)
	{
		bCanBePlacedInEngine = false;
	}

	virtual void CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor) override
	{
		// Rename module with "Generated" suffix
		FModuleDescriptor& ModuleDescriptor = Descriptor.Modules[0];
		ModuleDescriptor.Name = FName(ModuleDescriptor.Name.ToString() + TEXT("Generated"));

		// Add dependency on the WebAPI plugin
		Descriptor.Plugins.Add(FPluginReferenceDescriptor(TEXT("WebAPI"), true));
	}
};

FWebAPIPluginWizardDefinition::FWebAPIPluginWizardDefinition(bool bInIsContentOnlyProject)
	: bIsContentOnlyProject(bInIsContentOnlyProject)
{
	PluginBaseDir = IPluginManager::Get().FindPlugin(TEXT("WebAPI"))->GetBaseDir();

	PopulateTemplatesSource();
}

void FWebAPIPluginWizardDefinition::PopulateTemplatesSource()
{
	const FText GeneratedAPITemplateName = LOCTEXT("GeneratedAPILabel", "Generated API");	
	const FText GeneratedAPIDescription = LOCTEXT("GeneratedAPITemplateDesc", "Create a plugin to contain code generated by a WebAPI definition.");

	const TSharedRef<FPluginTemplateDescription> GeneratedAPITemplate = MakeShared<FWebAPIGeneratedPluginTemplateDescription>(GeneratedAPITemplateName, GeneratedAPIDescription, PluginBaseDir / TEXT("Templates") / TEXT("GeneratedAPI"), true, EHostType::Runtime);
	TemplateDefinitions.Add(GeneratedAPITemplate);

	TemplateDefinitions.Sort([](const TSharedRef<FPluginTemplateDescription>& A, const TSharedRef<FPluginTemplateDescription>& B)
	{
		return A->Name.CompareTo(B->Name) <= 0;
	});
}

const TArray<TSharedRef<FPluginTemplateDescription>>& FWebAPIPluginWizardDefinition::GetTemplatesSource() const
{
	return TemplateDefinitions;
}

void FWebAPIPluginWizardDefinition::OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	CurrentTemplateDefinition = InSelectedItem;
}

TSharedPtr<FPluginTemplateDescription> FWebAPIPluginWizardDefinition::GetSelectedTemplate() const
{
	return CurrentTemplateDefinition;
}

bool FWebAPIPluginWizardDefinition::HasValidTemplateSelection() const
{
	return CurrentTemplateDefinition.IsValid();
}

void FWebAPIPluginWizardDefinition::ClearTemplateSelection()
{
	CurrentTemplateDefinition.Reset();
}

bool FWebAPIPluginWizardDefinition::HasModules() const
{
	const FString SourceFolderPath = GetPluginFolderPath() / TEXT("Source");
	
	return FPaths::DirectoryExists(SourceFolderPath);
}

bool FWebAPIPluginWizardDefinition::IsMod() const
{
	return false;
}

FText FWebAPIPluginWizardDefinition::GetInstructions() const
{
	return LOCTEXT("ChoosePluginTemplate", "Choose a template and then specify a name to create a new plugin.");
}

bool FWebAPIPluginWizardDefinition::GetPluginIconPath(FString& OutIconPath) const
{
	return GetTemplateIconPath(CurrentTemplateDefinition.ToSharedRef(), OutIconPath);
}
 
EHostType::Type FWebAPIPluginWizardDefinition::GetPluginModuleDescriptor() const
{
	EHostType::Type ModuleDescriptorType = EHostType::Runtime;

	if (CurrentTemplateDefinition.IsValid())
	{
		ModuleDescriptorType = CurrentTemplateDefinition->ModuleDescriptorType;
	}

	return ModuleDescriptorType;
}

ELoadingPhase::Type FWebAPIPluginWizardDefinition::GetPluginLoadingPhase() const
{
	ELoadingPhase::Type Phase = ELoadingPhase::Default;

	if (CurrentTemplateDefinition.IsValid())
	{
		Phase = CurrentTemplateDefinition->LoadingPhase;
	}

	return Phase;
}

bool FWebAPIPluginWizardDefinition::GetTemplateIconPath(TSharedRef<FPluginTemplateDescription> Template, FString& OutIconPath) const
{
	bool bRequiresDefaultIcon = false;

	const FString TemplateFolderName = GetFolderForTemplate(Template);

	OutIconPath = TemplateFolderName / TEXT("Resources/Icon128.png");
	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*OutIconPath))
	{
		OutIconPath = PluginBaseDir / TEXT("Resources/DefaultIcon128.png");
		bRequiresDefaultIcon = true;
	}

	return bRequiresDefaultIcon;
}

TArray<FString> FWebAPIPluginWizardDefinition::GetFoldersForSelection() const
{
	TArray<FString> SelectedFolders;

	if (CurrentTemplateDefinition.IsValid())
	{
		SelectedFolders.Add(GetFolderForTemplate(CurrentTemplateDefinition.ToSharedRef()));
	}

	return SelectedFolders;
}

void FWebAPIPluginWizardDefinition::PluginCreated(const FString& PluginName, bool bWasSuccessful) const
{
	GEditor->GetEditorSubsystem<UWebAPIEditorSubsystem>()->OnModuleCreated().Broadcast(PluginName);
}

FString FWebAPIPluginWizardDefinition::GetPluginFolderPath() const
{
	return GetFolderForTemplate(CurrentTemplateDefinition.ToSharedRef());
}

FString FWebAPIPluginWizardDefinition::GetFolderForTemplate(TSharedRef<FPluginTemplateDescription> InTemplate) const
{
	return InTemplate->OnDiskPath;
}

#undef LOCTEXT_NAMESPACE
