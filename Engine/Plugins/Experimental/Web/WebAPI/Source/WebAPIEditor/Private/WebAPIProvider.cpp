// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIProvider.h"
#include "Async/Future.h"

TFuture<EWebAPIConversionResult> IWebAPIProviderInterface::ConvertToWebAPISchema(
	const TWeakObjectPtr<UWebAPIDefinition>& InDefinition)
{
	return MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::Failed).GetFuture();
}
