// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIProviderSettings.h"

#include "WebAPIEditorUtilities.h"
#include "Algo/AnyOf.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIService.h"

FString FWebAPIProviderSettings::MakeRequestTypeName(const FWebAPITypeNameVariant& InOperationName) const
{
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("OperationName"), InOperationName.ToString(true));
	
	return MakeValidMemberName(PopulateTemplate(RequestTypeNameTemplate, FormatArgs));
}

FString FWebAPIProviderSettings::MakeResponseTypeName(
	const FWebAPITypeNameVariant& InOperationName,
	const uint32& InResponseCode) const
{
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("OperationName"), InOperationName.ToString(true));
	FormatArgs.Add(TEXT("ResponseCode"), InResponseCode > 0 ? FString::FormatAsNumber(InResponseCode) : TEXT("Default"));
	
	return MakeValidMemberName(PopulateTemplate(ResponseTypeNameTemplate, FormatArgs));
}

FString FWebAPIProviderSettings::MakeNestedPropertyTypeName(
	const FWebAPITypeNameVariant& InModelName,
	const FWebAPINameVariant& InPropertyName) const
{
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ModelName"), InModelName.ToString(true));
	FormatArgs.Add(TEXT("PropertyName"),  ToPascalCase(InPropertyName.ToString(true)));
	
	return MakeValidMemberName(PopulateTemplate(NestedPropertyTypeNameTemplate, FormatArgs));
}

FString FWebAPIProviderSettings::MakeParameterTypeName(const FWebAPINameVariant& InParameterName) const
{
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ParameterName"), InParameterName.ToString(true));
	
	return MakeValidMemberName(PopulateTemplate(ParameterTypeNameTemplate, FormatArgs));
}

FName FWebAPIProviderSettings::MakeOperationObjectName(const TObjectPtr<UWebAPIService>& InService, const FWebAPITypeNameVariant& InOperationName) const
{
	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ClassName"), UWebAPIOperation::StaticClass()->GetName());
	FormatArgs.Add(TEXT("ServiceName"), InService->Name.ToString(true));
	FormatArgs.Add(TEXT("OperationName"), InOperationName.ToString(true));

	return MakeUniqueObjectName(InService, UWebAPIOperation::StaticClass(),
		FName(MakeValidMemberName(PopulateTemplate(OperationObjectNameTemplate, FormatArgs))));
}

const FString& FWebAPIProviderSettings::GetUnsetEnumValueName()
{
	return UnsetEnumValueName;
}

const FString& FWebAPIProviderSettings::GetDefaultPropertyName()
{
	return DefaultPropertyName;
}

const FString& FWebAPIProviderSettings::GetDefaultArrayPropertyName()
{
	return DefaultArrayPropertyName;
}

FString FWebAPIProviderSettings::ToPascalCase(const FWebAPINameVariant& InString) const
{
	return UE::WebAPI::FWebAPIStringUtilities::Get()->ToPascalCase(InString.ToString(true));
}

FString FWebAPIProviderSettings::ToInitials(const FWebAPINameVariant& InString) const
{
	return UE::WebAPI::FWebAPIStringUtilities::Get()->ToInitials(InString.ToString(true));
}

FString FWebAPIProviderSettings::MakeValidMemberName(const FWebAPINameVariant& InString, const FString& InPrefix) const
{
	return UE::WebAPI::FWebAPIStringUtilities::Get()->MakeValidMemberName(InString.ToString(true), InPrefix);
}

// @note: This assumes English words AND the word is plural
FString FWebAPIProviderSettings::Singularize(const FString& InString) const
{
	const TArray<FString> EsEnds = { TEXT("s"), TEXT("sh"), TEXT("ch"), TEXT("x"), TEXT("z") };

	if(InString.EndsWith(TEXT("es")))
	{
		FString Candidate = InString.LeftChop(2);
		if(Algo::AnyOf(EsEnds, [&](const FString& InEnd)
		{
			return InString.EndsWith(InEnd);
		}))
		{
			return Candidate;			
		}
	}

	if(InString.EndsWith(TEXT("ies")))
	{
		const FString Candidate = InString.LeftChop(3);
		return Candidate + TEXT("y");		
	}

	if(InString.EndsWith(TEXT("s")))
	{
		return InString.LeftChop(1);
	}

	return InString;
}

// @note: This assumes English words AND the word isn't already plural
FString FWebAPIProviderSettings::Pluralize(const FString& InString) const
{
	const TArray<FString> EsEnds = { TEXT("s"), TEXT("sh"), TEXT("ch"), TEXT("x"), TEXT("z") };

	if(Algo::AnyOf(EsEnds, [&](const FString& InEnd)
	{
		return InString.EndsWith(InEnd);
	}))
	{
		return InString + TEXT("es");
	}

	if(InString.EndsWith(TEXT("y")))
	{
		return InString.Left(InString.Len() - 1) + TEXT("ies");		
	}

	return InString + TEXT("s");
}

FString FWebAPIProviderSettings::PopulateTemplate(
	const FString& InTemplateString,
	const FStringFormatNamedArguments& InArgs) const
{
	return FString::Format(*InTemplateString, InArgs);
}
