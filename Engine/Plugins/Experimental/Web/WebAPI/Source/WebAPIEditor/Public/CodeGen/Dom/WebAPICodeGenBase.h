// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class WEBAPIEDITOR_API FWebAPICodeGenBase
{
public:	
	FWebAPICodeGenBase();
	virtual ~FWebAPICodeGenBase() = default;
	
	/** For Operations and Operation Parameters. */
	FString ServiceName;

	/** Description for this object. */
	FString Description;

	/** Containing module name. */
	FString Module;

	/** Type Namespace, can be empty for built-in types. */
	FString Namespace;

	/** U* specifiers, with or without a value. */
	TMap<FString, FString> Specifiers;

	/** U* metadata, with or without a value. */
	TMap<FString, FString> Metadata;

	/** Sets the containing module name. */
	virtual void SetModule(const FString& InModule);

	/** Retrieve the name. The source depends on the CodeGen object type. */
	virtual FString GetName(bool bJustName = false) { return TEXT(""); }

	/** Append any dependent modules for this file to the given Set. */
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const {}

	/** Append any include paths for this file to the given Array (not Set, to allow custom order). */
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const {}

	/** Append any include paths for this file to the given Set. */
	virtual void GetIncludePaths(TSet<FString>& OutIncludePaths) const;

	/** CodeGen Type. */
	virtual const FName& GetTypeName();
};
