// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenStruct.h"

class FWebAPICodeGenFunction;
class UWebAPIProperty;
class UWebAPIModel;

class WEBAPIEDITOR_API FWebAPICodeGenClass
	: public FWebAPICodeGenStruct
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenStruct;
	
public:
	FWebAPICodeGenClass(const FWebAPICodeGenStruct& InStruct);

	/** Functions within this class. */
	TArray<TSharedPtr<FWebAPICodeGenFunction>> Functions;

	/** Finds or creates a function with the given name. */
	const TSharedPtr<FWebAPICodeGenFunction>& FindOrAddFunction(const FWebAPINameVariant& InName);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Class");

	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
