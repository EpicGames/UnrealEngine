// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "Dom/WebAPISchema.h"

class UWebAPIEnumValue;
class UWebAPIEnum;

struct FWebAPICodeGenEnumValue
{
public:
	/** Name of the Enum Value. */
	FString Name;

	/** Display Name of the Enum Value. */
	FString DisplayName;

	/** Json Name of the Enum Value. */
	FString JsonName;

	/** Description of the Enum Value. */
	FString Description;

	/** Explicit integral value for an enum (0-255). */
	int32 IntegralValue = -1;

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const UWebAPIEnumValue* InSrcEnumValue);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("EnumValue");
	
	/** CodeGen Type. */
	const FName& GetTypeName() { return TypeName; }
};

class WEBAPIEDITOR_API FWebAPICodeGenEnum
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	/** Name of the Enum. */
	FWebAPITypeNameVariant Name;

	/** Values within the Enum. */
	TArray<FWebAPICodeGenEnumValue> Values;

	/** Default value as a string, if applicable. */
	FString DefaultValue;

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual FString GetName(bool bJustName = false) override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const UWebAPIEnum* InSrcEnum);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Enum");

	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
