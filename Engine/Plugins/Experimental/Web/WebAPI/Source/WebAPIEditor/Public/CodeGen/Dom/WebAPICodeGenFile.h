// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UWebAPIProperty;
class UWebAPIModel;

/** Contains a list of declarations/definitions to generate for a single file. */
class WEBAPIEDITOR_API FWebAPICodeGenFile
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	/** Absolute Base Path to output to, usually a Module/Source/x/(Public or Private). */
	FString BaseFilePath;
	
	/** Relative Path of the file, excluding the filename itself. */
	FString RelativeFilePath;

	/** Name of the file, excluding extension. */
	FString FileName;

 	/** File extension. */
	FString FileType;

	/** Copyright notice to apply to the top of the file. Uses the Project setting unless specified. */
	FString CopyrightNotice;

	/** Module dependencies for this file. */
	TSet<FString> Modules;

	/** Set of (unique) include paths. */
	TSet<FString> IncludePaths;
	
	/** The inner codegen items to create within the file. */
	TArray<TSharedPtr<FWebAPICodeGenBase>> SubItems;
	
	/** Schema objects represented by this file. */
	TArray<TWeakObjectPtr<UObject>> SchemaObjects;

	/** Adds the given item as a SubItem and appends include paths. */
	void AddItem(const TSharedPtr<FWebAPICodeGenBase>& InItem);

	/** Adds the given item as a SubItem and appends include paths. */
	template <typename CodeGenType>
	void AddItems(const TArray<TSharedPtr<CodeGenType>>& InItems);

	/** Returns the full, absolute path to the file. */
	FString GetFullPath();

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void SetModule(const FString& InModule) override;	
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual void GetIncludePaths(TSet<FString>& OutIncludePaths) const override;
	//~ End FWebAPICodeGenBase Interface.

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("File");
	
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};

template <typename CodeGenType>
void FWebAPICodeGenFile::AddItems(const TArray<TSharedPtr<CodeGenType>>& InItems)
{
	static_assert(std::is_base_of_v<FWebAPICodeGenBase, CodeGenType>, "CodeGenType must be derived from FWebAPICodeGenBase");

	for(const TSharedPtr<CodeGenType>& Item : InItems)
	{
		AddItem(Item);
	}
}
