// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "WebAPICodeGenProperty.h"

class WEBAPIEDITOR_API FWebAPICodeGenFunctionParameter
	: public FWebAPICodeGenProperty
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenProperty;
	
public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("FunctionParameter");};

/** */
class WEBAPIEDITOR_API FWebAPICodeGenFunction
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Function");
	
	/** Name with optional prefix, namespace, etc. */
	FWebAPINameVariant Name;

	/** Flag for a constant function. */
	bool bIsConst = false;

	/** Flag if the function is a virtual override. */
	bool bIsOverride = false;

	/** The type to return. If unspecified it will be void. */
	FWebAPITypeNameVariant ReturnType;

	/** Flag if the returned value is constant. */
	bool bIsConstReturnType = false;

	/** Parameters for this function. */
	TArray<TSharedPtr<FWebAPICodeGenFunctionParameter>> Parameters;

	/** Raw string to insert as the Function implementation. */
	FString Body;
};
