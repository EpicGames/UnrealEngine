// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "WebAPICodeGenStruct.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPISchema.h"

class FWebAPICodeGenStruct;

class WEBAPIEDITOR_API FWebAPICodeGenOperationParameter
	: public FWebAPICodeGenProperty
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenProperty;
	
public:
	/** Where this parameter is stored/encoded in the request. */
	EWebAPIParameterStorage Storage;

	/** The optional media-type, ie. "application/json". */
	FString MediaType;

	/** Mark as required (not-optional). */
	bool bIsRequired;

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const class UWebAPIOperationParameter* InSrcOperationParameter);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("OperationParameter");
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};

/** Encapsulates the request parameters for an operation. */
class WEBAPIEDITOR_API FWebAPICodeGenOperationRequest
	: public FWebAPICodeGenStruct
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenStruct;
	
public:
	/** Array of parameters contained in this request. */
	TArray<TSharedPtr<FWebAPICodeGenOperationParameter>> Parameters;

	/** Finds or creates a parameter with the given name. */
	const TSharedPtr<FWebAPICodeGenOperationParameter>& FindOrAddParameter(const FString& InName);

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual void SetModule(const FString& InModule) override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const class UWebAPIOperationRequest* InSrcOperationRequest);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("OperationRequest");
	
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};

class WEBAPIEDITOR_API FWebAPICodeGenOperationResponse
	: public FWebAPICodeGenStruct
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenStruct;
	
public:
	/** Where this parameter is stored/encoded in the response. */
	EWebAPIResponseStorage Storage;
	
	/** Http response code. */
	uint32 ResponseCode;

	/** Optional message based on the response status or result. */
	FString Message;

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual void SetModule(const FString& InModule) override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const class UWebAPIOperationResponse* InSrcOperationResponse);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("OperationResponse");
	
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};

class WEBAPIEDITOR_API FWebAPICodeGenOperation
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	/** Name with optional prefix, namespace, etc. */
	FWebAPITypeNameVariant Name;

	/** Http Verb. */
	FString Verb;

	/** Templated/tokenized Url path relative to the Base Url. */
	FString Path;
	
	/** Usually application/json. */
	FString RequestMediaType;

	/** Flags an operation previously available as deprecated. */ 
	bool bIsDeprecated = false;

	/** Requests for this Operation. */
	TArray<TSharedPtr<FWebAPICodeGenOperationRequest>> Requests;

	/** Responses for this Operation. */
	TArray<TSharedPtr<FWebAPICodeGenOperationResponse>> Responses;

	/** The name of the generated Settings class. */
	FString SettingsTypeName;

	/** Finds or creates a response with the given name. */
	const TSharedPtr<FWebAPICodeGenOperationResponse>& FindOrAddResponse(const uint32& InResponseCode);

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual void SetModule(const FString& InModule) override;
	virtual FString GetName(bool bJustName = false) override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const class UWebAPIOperation* InSrcOperation);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Operation");
	
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
