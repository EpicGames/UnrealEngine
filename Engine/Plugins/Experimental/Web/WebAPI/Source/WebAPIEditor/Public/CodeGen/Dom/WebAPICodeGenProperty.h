// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "Dom/WebAPISchema.h"

class WEBAPIEDITOR_API FWebAPICodeGenProperty
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	FWebAPICodeGenProperty();
	
	/** Name with optional prefix, namespace, etc. */
	FWebAPINameVariant Name;
	
	/** Property Type. */
	FWebAPITypeNameVariant Type;

	/** Constant declaration. */
	bool bIsConst = false;

	/** Default value as a string, if applicable. */
	FString DefaultValue;

	/** If the property is an array of Type. */
	bool bIsArray = false;

	/** If the property type is the same as it's container. */
	bool bIsRecursiveType = false;

	/** By default all properties are optional. */
	bool bIsRequired = false;

	/** When the properties inside of this should be treated as though they're directly in the parent. */
	bool bIsMixin = false;

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const class UWebAPIProperty* InSrcProperty);
	
public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Property");	

	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
