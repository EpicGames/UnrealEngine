// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "WebAPIDefinition.h"
#include "Dom/WebAPISchema.h"

/** A settings class is generated from this, it's not in itself a Settings object. */
class WEBAPIEDITOR_API FWebAPICodeGenSettings
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:	
	/** Name with optional prefix, namespace, etc. */
	FWebAPITypeNameVariant Name;

	/** Optional parent type. Useful if you have your own settings for this API that you want to extend. */
	TSoftClassPtr<UDeveloperSettings> ParentType = UDeveloperSettings::StaticClass();

	/** The default host address to access this API. */
	FString Host;

	/** The Url path relative to the host address, ie. "/V1". */
	FString BaseUrl;

	/** The UserAgent to encode in Http request headers. */
	FString UserAgent = TEXT("X-UnrealEngine-Agent");

	/** The date-time format this API uses to encode/decode from string. */
	FString DateTimeFormat = TEXT("");

	/** Uniform Resource Identifier schemes (ie. https, http). */
	TArray<FString> Schemes;

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	//~ End FWebAPICodeGenBase Interface.

	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const UWebAPIDefinition* InSrcModel);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Settings");

	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
