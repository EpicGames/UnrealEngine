// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPICodeGenBase.h"
#include "WebAPICodeGenProperty.h"
#include "Dom/WebAPISchema.h"

class UWebAPIProperty;
class UWebAPIModel;

class WEBAPIEDITOR_API FWebAPICodeGenStruct
	: public FWebAPICodeGenBase
{
	/** Inherited baseclass. */
	using Super = FWebAPICodeGenBase;
	
public:
	/** Name with optional prefix, namespace, etc. */
	FWebAPITypeNameVariant Name;

	/** Base class. Should be empty if none. */
	FWebAPITypeNameVariant Base;

	/** Properties within this struct. */
	TArray<TSharedPtr<FWebAPICodeGenProperty>> Properties;

	/** Finds or creates a property with the given name. */
	const TSharedPtr<FWebAPICodeGenProperty>& FindOrAddProperty(const FWebAPINameVariant& InName);

	//~ Begin FWebAPICodeGenBase Interface.
	virtual void GetModuleDependencies(TSet<FString>& OutModules) const override;
	virtual void GetIncludePaths(TArray<FString>& OutIncludePaths) const override;
	virtual FString GetName(bool bJustName = false) override;
	//~ End FWebAPICodeGenBase Interface.

	/** Find and return properties that are of the same type as the containing class. */
	bool FindRecursiveProperties(TArray<TSharedPtr<FWebAPICodeGenProperty>>& OutProperties);
	
	/** Populates this object from the given WebAPI object. */
	void FromWebAPI(const UWebAPIModel* InSrcModel);

public:
	/** CodeGen Type. */
	inline static const FName TypeName = TEXT("Struct");
	
	/** CodeGen Type. */
	virtual const FName& GetTypeName() override { return TypeName; }
};
