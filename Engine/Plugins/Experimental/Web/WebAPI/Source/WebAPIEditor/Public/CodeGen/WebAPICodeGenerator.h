// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "Dom/WebAPICodeGenClass.h"
#include "Dom/WebAPICodeGenEnum.h"
#include "Dom/WebAPICodeGenFile.h"
#include "Dom/WebAPICodeGenOperation.h"
#include "Dom/WebAPICodeGenStruct.h"
#include "UObject/Interface.h"

#include "WebAPICodeGenerator.generated.h"

class UWebAPIDefinition;

enum class EWebAPIGenerationResult : uint8
{
	Failed = 0,
	Succeeded = 1,
	FailedWithErrors = 2,
	FailedWithWarnings = 3,
	SucceededWithWarnings = 4
};

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class UWebAPICodeGeneratorInterface : public UInterface
{
	GENERATED_BODY()
};

/** Interface for code generation for a given WebAPI Definition/Schema. */
class WEBAPIEDITOR_API IWebAPICodeGeneratorInterface
{
	GENERATED_BODY()
	
public:
	/** Check to see if this generator is available for use. */
	virtual TFuture<bool> IsAvailable() = 0;
	
	/** Generate code for the given WebAPI Definition. */
	virtual TFuture<EWebAPIGenerationResult> Generate(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition) = 0;
};

/** WebAPICodeGenerator base implementation. Converts WebAPI schema to code gen primitives. Use of this baseclass is optional. */
UCLASS(Abstract)
class WEBAPIEDITOR_API UWebAPICodeGeneratorBase
	: public UObject
	, public IWebAPICodeGeneratorInterface
{
	GENERATED_BODY()
	
public:
	/** Check to see if this generator is available for use. */
	virtual TFuture<bool> IsAvailable() override;
	
	/** Generate code for the given WebAPI Definition. */
	virtual TFuture<EWebAPIGenerationResult> Generate(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition) override;

	/** Converts the given Service to a CodeGen Operation array. */
	virtual TFuture<EWebAPIGenerationResult> GenerateServiceOperations(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const FString& InServiceName, const TArray<TSharedPtr<FWebAPICodeGenOperation>>& InOperations);
	
	/** Converts the given Enum to a CodeGen Enum object. */
	virtual TFuture<EWebAPIGenerationResult> GenerateEnum(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenEnum>& InEnum);

	/** Converts the given Model to a CodeGen Struct object. */
	virtual TFuture<EWebAPIGenerationResult> GenerateModel(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenStruct>& InStruct);

	/** Converts the given Definition to a CodeGen File object. */
	virtual TFuture<EWebAPIGenerationResult> GenerateFile(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenFile>& InFile);

	/** Converts the given Definition to a CodeGen Settings object. */
	virtual TFuture<EWebAPIGenerationResult> GenerateSettings(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenClass>& InSettingsClass);

public:
	static constexpr const TCHAR* LogName = TEXT("Code Generator");
	
protected:
	template <typename CodeGenType>
	bool CheckNameConflicts(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TArray<TSharedPtr<CodeGenType>>& InItems) const;

	template <>
	bool CheckNameConflicts(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TArray<TSharedPtr<FWebAPICodeGenFile>>& InItems) const;
};
