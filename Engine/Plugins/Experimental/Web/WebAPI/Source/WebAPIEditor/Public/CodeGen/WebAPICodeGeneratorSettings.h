// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIDeveloperSettings.h"
#include "UObject/Object.h"

#include "WebAPICodeGeneratorSettings.generated.h"

/** Contains information about the target module for code generation files. */
USTRUCT()
struct FWebAPIDefinitionTargetModule
{
	GENERATED_BODY()

	/** The name of the module. */
	UPROPERTY(EditAnywhere, Category = "Module")
	FString Name;

	/** The absolute location of the directory for this module. Only use to set, otherwise use GetPath(). */
	UPROPERTY(EditAnywhere, Category = "Module")
	FString AbsolutePath;

	/** Resolves and returns the path, use instead of AbsolutePath. */
	WEBAPIEDITOR_API FString GetPath();

private:
	/** Caches the full path to the module. */
	UPROPERTY(Transient)
	FString ResolvedPath;
};

/** Encapsulates a new pseudo-namespace. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNamespaceChangedDelegate, const FString& /* NewNamespace */);

/** Encapsulates settings for WebAPI code generation. */
USTRUCT()
struct WEBAPIEDITOR_API FWebAPICodeGeneratorSettings
{
	GENERATED_BODY()

	FWebAPICodeGeneratorSettings();

	/** Whether to override the Code Generator class specified in project settings. */
	UPROPERTY(EditAnywhere, Category = "Generator", meta = (InlineEditConditionToggle))
	bool bOverrideGeneratorClass = false;

	/** The Code Generator to use. */
	UPROPERTY(EditAnywhere, NoClear, Category = "Generator", meta = (MustImplement = "/Script/WebAPIEditor.WebAPICodeGeneratorInterface", EditCondition="bOverrideGeneratorClass"))
	TSoftClassPtr<UObject> CodeGeneratorClass;

	/** The copyright notice to apply to generated files. */
	UPROPERTY(EditAnywhere, Category = "Generator")
	FString CopyrightNotice;

	/** The C++ Unreal module determines the location of the generated files. */
	UPROPERTY(EditAnywhere, Category = "Generator")
	FWebAPIDefinitionTargetModule TargetModule;

	/** The relative output path for generated models. This accepts the token "{Model}", where Model is the name of the generated object. */
	UPROPERTY(EditAnywhere, Category = "Generator")
	FString ModelOutputPath = TEXT("Models");

	/** The relative output path for generated operations. This accepts the token "{Service}" and "{Operation}". */
	UPROPERTY(EditAnywhere, Category = "Generator")
	FString OperationOutputPath = TEXT("Services/{Service}");

	/** The generated settings baseclass (optional). */
	UPROPERTY(EditAnywhere, NoClear, AdvancedDisplay, Category = "Generator", meta = (AllowAbstract))
	TSoftClassPtr<UWebAPIDeveloperSettings> GeneratedSettingsBaseClass = UWebAPIDeveloperSettings::StaticClass();

#if WITH_EDITORONLY_DATA
	/** Disable to prevent the generator writing files/assets. The generation process itself will still occur. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Generator")
	bool bWriteResult = true;
#endif

	/** Resolves and returns an instance of the CodeGeneratorClass. */
	TScriptInterface<class IWebAPICodeGeneratorInterface> GetGeneratorClass() const;

	/** Returns the specified pseudo-namespace. */
	const FString& GetNamespace() const;

	/** Sets the specified pseudo-namespace. */
	void SetNamespace(const FString& InNamespace);

	/** When the pseudo-namespace changes. */
	FOnNamespaceChangedDelegate& OnNamespaceChanged();

private:
	/** A pseudo-namespace to prefix generated types with to avoid naming conflicts. Usually the API name. */
	UPROPERTY(EditAnywhere, Category = "Generator", meta = (AllowPrivateAccess))
	FString Namespace;

	/** When the pseudo-namespace changes. */
	FOnNamespaceChangedDelegate OnNamespaceChangedDelegate;
};
