// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPISchema.h"

#include "WebAPICompositeModel.generated.h"

UENUM()
enum class EWebAPIModelCompositionType
{
	/** Should contain *one* of the valid types. Maps to a Variant. */
	Single		UMETA(DisplayName = "Single"),
	/** Should contain *one or more* of the valid types. */
	Multiple	UMETA(DisplayName = "Multiple"),
	/** Should contain *all* of the valid types. */
	All			UMETA(DisplayName = "All"),
};

namespace UE
{
	namespace WebAPI
	{
		namespace WebAPIModelCompositionType
		{
			WEBAPIEDITOR_API const FString& ToString(const EWebAPIModelCompositionType& InEnumValue);
			WEBAPIEDITOR_API const EWebAPIModelCompositionType& FromString(const FString& InStringValue);	
		}
	}
}

UCLASS(MinimalAPI)
class UWebAPICompositeModel
	: public UWebAPIModelBase
{
	GENERATED_BODY()
	
public:
	/** Name of the composition. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Name;	

	/** Type of composition. */
	UPROPERTY(EditAnywhere, Category = "Type")
	EWebAPIModelCompositionType CompositionType;

	/** Array of one or more types contained in the composition. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<FWebAPITypeNameVariant> Types;

#if WITH_EDITORONLY_DATA
	/** The last generated code as text for debugging. */
	UPROPERTY(Transient)
	FString GeneratedCodeText;
#endif

	/** Used for sorting in the UI. */
	virtual FString GetSortKey() const override;

	/** Recursively sets the namespace of this and all child objects.  */
	virtual void SetNamespace(const FString& InNamespace) override;
};
