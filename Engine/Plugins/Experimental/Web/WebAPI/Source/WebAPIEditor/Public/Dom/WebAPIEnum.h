// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/WebAPISchema.h"

#include "WebAPIEnum.generated.h"

class FJsonObject;

/** Describes a single value within a enum. */
UCLASS(MinimalAPI)
class UWebAPIEnumValue
	: public UWebAPIModelBase
{
	GENERATED_BODY()

public:
	/** Name of the EnumValue. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPINameVariant Name;
};

/** Describes a (usually) API-specific class representing an enum. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIEnum
	: public UWebAPIModelBase
{
	GENERATED_BODY()
	
public:
	/** Name of the Enum. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Name;
	
	/** Will be set based on dependent operations. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bGenerate;

	/** The enum base type, always uint8. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Type = TEXT("uint8");
	
	/** Current or Default Value of the Enum. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString DefaultValue;

	/** Values within the Enum. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIEnumValue>> Values;

#if WITH_EDITORONLY_DATA
	/** The last generated code as text for debugging. */
	UPROPERTY(Transient)
	FString GeneratedCodeText;
#endif

	/** Add and return a reference to a new Enum Value. */
	TObjectPtr<UWebAPIEnumValue> AddValue();

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual FString GetSortKey() const override;
	virtual void SetNamespace(const FString& InNamespace) override;
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;	
	virtual bool ToJson(TSharedPtr<FJsonObject>& OutJson) override;
	virtual void BindToTypeInfo() override;
#if WITH_EDITOR
	virtual void SetCodeText(const FString& InCodeText) override;
	virtual void AppendCodeText(const FString& InCodeText) override;
#endif
	//~ End IWebAPISchemaObjectInterface Interface.

	/** Default value as a string, usually the first enum value. */
	FString GetDefaultValue(bool bQualified = false) const;
};
