// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/WebAPISchema.h"

#include "WebAPIModel.generated.h"

class UWebAPIEnum;

/** Describes a single property within a model. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIProperty
	: public UWebAPIModelBase
{
	GENERATED_BODY()
	
public:
	/** Name of the Property. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPINameVariant Name;

	/** Type of the Property. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Type;

	/** If the property is an array of Type. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bIsArray = false;

	/** When the properties inside of this should be treated as though they're directly in the parent. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bIsMixin = false;

	/** Default value (optional) as a string. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString DefaultValue;
	
	/** Array of values if needed, ie. enum */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<FString> DefaultValues;

	/** Returns if this property is required to have it's value set by the user. */
	virtual bool IsRequired() const;

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual void SetNamespace(const FString& InNamespace) override;
	//~ End IWebAPISchemaObjectInterface Interface.

	/** Default value as a string, if applicable. */
	FString GetDefaultValue(bool bQualified = false) const;
	
#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
};

/** Describes a (usually) API-specific struct or class representing a model. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIModel
	: public UWebAPIModelBase
{
	GENERATED_BODY()
	
public:
	/** Name of the Model. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Name;

	/** Will be set based on dependent operations. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	bool bGenerate = true;

	/** The corresponding TypeInfo for this Model. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Type;

	/** Default value (optional) as a string. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString DefaultValue;

	/** Array of properties contained in this model. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIProperty>> Properties;

#if WITH_EDITORONLY_DATA
	/** The last generated code as text for debugging. */
	UPROPERTY(Transient)
	FString GeneratedCodeText;
#endif

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual FString GetSortKey() const override;
	virtual void SetNamespace(const FString& InNamespace) override;
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;
	virtual void BindToTypeInfo() override;
#if WITH_EDITOR
	virtual void SetCodeText(const FString& InCodeText) override;
	virtual void AppendCodeText(const FString& InCodeText) override;
#endif
	//~ End IWebAPISchemaObjectInterface Interface.

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
};
