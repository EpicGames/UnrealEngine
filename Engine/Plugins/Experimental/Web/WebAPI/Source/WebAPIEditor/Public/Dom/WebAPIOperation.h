// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/WebAPIModel.h"
#include "Dom/WebAPISchema.h"
#include "UObject/Object.h"

#include "WebAPIOperation.generated.h"

UENUM()
enum class EWebAPIParameterStorage
{
	/** /users/{value} */
	Path UMETA(DisplayName = "Path"),
	/** /users?{key}={value}} */
	Query UMETA(DisplayName = "Query"),
	/** {key}: {value} */
	Header UMETA(DisplayName = "Header"),
	/** {key}={value} */
	Cookie UMETA(DisplayName = "Cookie"),
	/** ie. json string */
	Body UMETA(DisplayName = "Body")
};

UENUM()
enum class EWebAPIResponseStorage
{
	/** ie. json string */
	Body UMETA(DisplayName = "Body"),
	/** {key}: {value} */
	Header UMETA(DisplayName = "Header"),
};

namespace UE::WebAPI::WebAPIParameterStorage
{
	WEBAPIEDITOR_API const FString& ToString(const EWebAPIParameterStorage& InEnumValue);
	WEBAPIEDITOR_API const EWebAPIParameterStorage& FromString(const FString& InStringValue);
}

UCLASS(MinimalAPI)
class UWebAPIOperationParameter
	: public UWebAPIProperty
{
	GENERATED_BODY()

public:
	/** Where this parameter is stored/encoded in the request. */
	UPROPERTY(EditAnywhere, Category = "Type")
	EWebAPIParameterStorage Storage;

	/** The optional media-type, ie. "application/json". */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString MediaType;

	/** Optional model definition. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TObjectPtr<UWebAPIModel> Model;

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual bool IsRequired() const override;
	virtual void SetNamespace(const FString& InNamespace) override;
	//~ End IWebAPISchemaObjectInterface Interface.
};

UCLASS(MinimalAPI)
class UWebAPIOperationRequest
	: public UWebAPIModel
{
	GENERATED_BODY()

public:
	/** Array of parameters contained in this request. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIOperationParameter>> Parameters;

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual void SetNamespace(const FString& InNamespace) override;
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;
	//~ End IWebAPISchemaObjectInterface Interface.
};

UCLASS(MinimalAPI)
class UWebAPIOperationResponse
	: public UWebAPIModel
{
	GENERATED_BODY()

public:
	/** Where this parameter is stored/encoded in the response. */
	UPROPERTY(EditAnywhere, Category = "Type")
	EWebAPIResponseStorage Storage;

	/** Http response code. */
	UPROPERTY(EditAnywhere, Category = "Type")
	uint32 Code = 200; // ie. 200

	/** Optional message based on the response status or result. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Message;
};

UCLASS(MinimalAPI)
class UWebAPIOperationError
	: public UObject
	, public IWebAPISchemaObjectInterface
{
	GENERATED_BODY()

public:
	/** Http response code. */
	UPROPERTY(EditAnywhere, Category = "Type")
	uint32 Code = 404;

	/** Describes this error, usually the error message. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Description;
};

UCLASS()
class WEBAPIEDITOR_API UWebAPIOperation
	: public UObject
	, public IWebAPISchemaObjectInterface
{
	GENERATED_BODY()

public:
	UWebAPIOperation();

	/** Name of the Operation. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Name;

	/** Can be disabled to skip generation of this Operation. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bGenerate = true;

	/** Parent service for this operation. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TObjectPtr<class UWebAPIService> Service;

	/** Describes this operation. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Description;

	/** Flags an operation previously available as deprecated. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bIsDeprecated = false;

	/** The Http verb, ie. Get, Post, etc. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Verb;

	/** The operation path, relative to the base url. This can contain named template args. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString Path;

	/** List of possible content types for requests, ie. application/json */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<FString> RequestContentTypes;

	/** List of possible content types for responses, ie. application/json */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<FString> ResponseContentTypes;

	/** Request model. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TObjectPtr<UWebAPIOperationRequest> Request;

	/** Array of responses. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIOperationResponse>> Responses;

	/** Array of errors that can be returned. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIOperationError>> Errors;

#if WITH_EDITORONLY_DATA
	/** The last generated code as text for debugging. */
	UPROPERTY(Transient)
	FString GeneratedCodeText;
#endif

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual void SetNamespace(const FString& InNamespace) override;
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;
#if WITH_EDITOR
	virtual void SetCodeText(const FString& InCodeText) override;
	virtual void AppendCodeText(const FString& InCodeText) override;
#endif
	//~ End IWebAPISchemaObjectInterface Interface.

	/** Associates this model with it's own TypeInfo. */
	virtual void BindToTypeInfo();

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
};
