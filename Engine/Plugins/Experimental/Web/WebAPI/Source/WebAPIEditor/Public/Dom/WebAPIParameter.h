// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIOperation.h"

#include "WebAPIParameter.generated.h"

/** A re-usable Parameter. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIParameter
	: public UWebAPIModel
{
	GENERATED_BODY()

public:
	/** Where this parameter is stored/encoded in the request. */
	UPROPERTY(EditAnywhere, Category = "Type")
	EWebAPIParameterStorage Storage;

	/** The optional media-type, ie. "application/json". */
	UPROPERTY(EditAnywhere, Category = "Type")
	FString MediaType;

	/** Optional model definition. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TObjectPtr<UWebAPIModel> Model;

	/** If the property is an array of Type. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bIsArray = false;

	/** Single value property that this Parameter wraps. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TObjectPtr<UWebAPIProperty> Property;

	//~ Begin IWebAPISchemaObjectInterface Interface.
#if WITH_EDITOR
	virtual void SetCodeText(const FString& InCodeText) override;
	virtual void AppendCodeText(const FString& InCodeText) override;
#endif
	//~ End IWebAPISchemaObjectInterface Interface.
};
