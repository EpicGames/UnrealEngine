// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIMessageLog.h"
#include "Dom/JsonObject.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"

#include "WebAPISchema.generated.h"

class UWebAPIModel;
class UWebAPIProperty;
class UWebAPITypeInfo;
class UWebAPITypeRegistry;

UINTERFACE(MinimalAPI, BlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UWebAPISchemaObjectInterface : public UInterface
{
	GENERATED_BODY()
};

class WEBAPIEDITOR_API IWebAPISchemaObjectInterface
{
	GENERATED_BODY()

public:
	/** Used for sorting in the UI. */
	virtual FString GetSortKey() const;
	
	/** Recursively sets the namespace of this and all child objects. */
	virtual void SetNamespace(const FString& InNamespace);

	/** Recursively visit each element in the schema. */
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor);

	virtual bool ToJson(TSharedPtr<FJsonObject>& OutJson);

#if WITH_EDITOR
	/** Sets the last generated code as text for debugging. */
	virtual void SetCodeText(const FString& InCodeText);

	/** Appends the last generated code as text for debugging. */
	virtual void AppendCodeText(const FString& InCodeText);
#endif
};

/** Holds either UWebAPITypeInfo or FString. */
USTRUCT(BlueprintType)
struct WEBAPIEDITOR_API FWebAPITypeNameVariant
{
	GENERATED_BODY()

	/** TypeInfo. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	TSoftObjectPtr<UWebAPITypeInfo> TypeInfo = nullptr;

	/** String. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString TypeString;

	/** Field name as sent to and received from the external API. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString JsonName;

	FWebAPITypeNameVariant() = default;

	FWebAPITypeNameVariant(const TObjectPtr<UWebAPITypeInfo>& InTypeName);
	
	FWebAPITypeNameVariant(const FString& InStringName);

	FWebAPITypeNameVariant(const TCHAR* InStringName);

	/** Check if this has valid TypeInfo. */
	bool HasTypeInfo() const;

	/** Check that either TypeInfo is set and valid, or a TypeString is non-empty. */
	bool IsValid() const;
	
	/** Get the fully formatted name as a valid member name. Optionally provide a prefix for invalid member names (ie. that start with a number). */
	FString ToMemberName(const FString& InPrefix = {}) const;

	/** Retrieve the String of whichever property is valid. */
	FString ToString(bool bInJustName = false) const;

	FString GetDisplayName() const;

	FString GetJsonName() const;

	/** Copy-assignment. */
	FWebAPITypeNameVariant& operator=(const FString& InStringName)
	{
		TypeString = InStringName;
		return *this;	
	}

	// Comparisons.
	friend bool operator==(const FWebAPITypeNameVariant& A, const FWebAPITypeNameVariant& B)
	{
		return A.HasTypeInfo() && B.HasTypeInfo()
		? A.TypeInfo == B.TypeInfo
		: A.ToString(true) == B.ToString(true);
	}

	friend bool operator!=(const FWebAPITypeNameVariant& A, const FWebAPITypeNameVariant& B)
	{
		return !(A == B);
	}
};

/** Holds information for a name and it's alternatives. */
USTRUCT(BlueprintType)
struct WEBAPIEDITOR_API FWebAPINameInfo
{
	GENERATED_BODY()

	/** Name without prefix or namespace, ie. "SomeParameter", not "InSomeParameter". */
	UPROPERTY(VisibleAnywhere, Category = "Name")
	FString Name;

	/** Field name as sent to and received from the external API. */
	UPROPERTY(VisibleAnywhere, Category = "Name")
	FString JsonName;

	/** Type Prefix, usually for parameters ("In"/"Out") or booleans "b". */
	UPROPERTY(VisibleAnywhere, Category = "Name")
	FString Prefix;

	FWebAPINameInfo() = default;

	/** Optionally provide TypeInfo to infer a prefix, for example "b" for booleans. */
	explicit FWebAPINameInfo(const FString& InName, const FString& InJsonName, const FWebAPITypeNameVariant& InTypeInfo = {});

	/** Get the fully formatted name, with prefix. */
	FString ToString(bool bInJustName = false) const;

	/** Get the fully formatted name as a valid member name. */
	FString ToMemberName(const FString& InPrefix = {}) const;

	/** The name for UI display. */
	FString GetDisplayName() const;

	/** The json field name that the target API expects, rather than the UE friendly one. */
	FString GetJsonName() const;
	
	// Comparisons.
	friend bool operator==(const FWebAPINameInfo& A, const FWebAPINameInfo& B)
	{
		return A.Name == B.Name;
	}

	friend bool operator!=(const FWebAPINameInfo& A, const FWebAPINameInfo& B)
	{
		return !(A == B);
	}
};

/** Holds either FWebAPINameInfo or FString. */
USTRUCT(BlueprintType)
struct WEBAPIEDITOR_API FWebAPINameVariant
{
	GENERATED_BODY()

	/** NameInfo. */
	UPROPERTY(VisibleAnywhere, Category = "Name")
	FWebAPINameInfo NameInfo;

	/** String. */
	UPROPERTY(VisibleAnywhere, Category = "Name")
	FString NameString;

	FWebAPINameVariant() = default;

	FWebAPINameVariant(const FWebAPINameInfo& InNameInfo);

	FWebAPINameVariant(const FString& InStringName);

	FWebAPINameVariant(const TCHAR* InStringName);

	/** Check if this has a valid NameInfo. */
	bool HasNameInfo() const;

	/** Check that either NameInfo is set and valid, or a NameString is non-empty. */
	bool IsValid() const;

	/** Retrieve the String of whichever property is valid. */
	FString ToString(bool bInJustName = false) const;

	/** Get the fully formatted name as a valid member name. Optionally provide a prefix for invalid member names (ie. that start with a number). */
	FString ToMemberName(const FString& InPrefix = {}) const;

	/** The name for UI display. */
	FString GetDisplayName() const;

	/** The json field name that the target API expects, rather than the UE friendly one. */
	FString GetJsonName() const;

	/** Copy-assignment. */
	FWebAPINameVariant& operator=(const FString& InStringName);

	// Comparisons.
	friend bool operator==(const FWebAPINameVariant& A, const FWebAPINameVariant& B)
	{
		return A.HasNameInfo() && B.HasNameInfo()
		? A.NameInfo == B.NameInfo
		: A.ToString(true) == B.ToString(true);
	}

	friend bool operator!=(const FWebAPINameVariant& A, const FWebAPINameVariant& B)
	{
		return !(A == B);
	}
};

/** Baseclass with common properties for various Schema classes. */
UCLASS(MinimalAPI, Abstract)
class UWebAPIModelBase
	: public UObject
	, public IWebAPISchemaObjectInterface
{
	GENERATED_BODY()
	
public:
	/** Describes this model. */
	UPROPERTY(VisibleAnywhere, Category = "Model")
	FString Description;

	/** By default all properties are optional. */
	UPROPERTY(VisibleAnywhere, Category = "Model")
	bool bIsRequired = false;

	/** By default all properties are read & write. */
	UPROPERTY(VisibleAnywhere, Category = "Model")
	bool bIsReadOnly = false;

	/** Whether to use Minimum value. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (InlineEditConditionToggle))
	bool bUseMinimumValue;

	/** Minimum value. Can also indicate minimum string length. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (EditCondition="bUseMinimumValue"))
    double MinimumValue;

	/** Whether to use Maximum value. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (InlineEditConditionToggle))
	bool bUseMaximumValue;

	/** Maximum value. Can also indicate maximum string length. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (EditCondition="bUseMaximumValue"))
	double MaximumValue;

	/** Whether to use a Regex Pattern for validation. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (InlineEditConditionToggle))
	bool bUsePattern;

	/** Regex Pattern to validate against. */
	UPROPERTY(VisibleAnywhere, Category = "Model", meta = (EditCondition="bUsePattern"))
	FString Pattern;

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual void SetNamespace(const FString& InNamespace) override;
	//~ End IWebAPISchemaObjectInterface Interface.

	/** Associates this model with it's own TypeInfo. */
	virtual void BindToTypeInfo();

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
};

/** Describes the intermediate structure from which to generate code. */
UCLASS(BlueprintType)
class WEBAPIEDITOR_API UWebAPISchema
	: public UObject
	, public IWebAPISchemaObjectInterface 
{
	GENERATED_BODY()

public:
	UWebAPISchema();

	/** Name of the API. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	FString APIName;

	/** The API version can be any arbitrary string that is useful if there are several versions. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	FString Version;

	/** The default host address to access this API. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	FString Host;

	/** The Url path relative to the host address, ie. "/V1". */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	FString BaseUrl;

	/** The UserAgent to encode in Http request headers. */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Schema")
	FString UserAgent = TEXT("X-UnrealEngine-Agent");

	/** The date-time format this API uses to encode/decode from string. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	FString DateTimeFormat;

	/** Uniform Resource Identifier schemes (ie. https, http). */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	TArray<FString> URISchemes;

	/** Contains all referenced types for this API. */
	UPROPERTY()
	TObjectPtr<UWebAPITypeRegistry> TypeRegistry;

	/** Models can be a combination of classes, structs and enums used by this API. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	TArray<TObjectPtr<UWebAPIModelBase>> Models;

	/** Each service contains one or more operations. */
	UPROPERTY(VisibleAnywhere, Category = "Schema")
	TMap<FString, TObjectPtr<class UWebAPIService>> Services;

	/** Add and return a reference to a new Model */
	template <typename ModelType>
	ModelType* AddModel(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo = {});

	/** Add and return a reference to a new Enum. */
	class UWebAPIEnum* AddEnum(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo = {});

	/** Add and return a reference to a new Parameter. These can be referenced by operations rather than defining them inline. */
	class UWebAPIParameter* AddParameter(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo);

	/** Find/Add and return a reference to an existing/new Service. */
	TObjectPtr<class UWebAPIService> GetOrMakeService(const FString& InName);

	/** Resets the Schema to defaults. */
	void Clear();

	//~ Begin IWebAPISchemaObjectInterface Interface.
	const TSharedPtr<FWebAPIMessageLog>& GetMessageLog() const;
	//~ End IWebAPISchemaObjectInterface Interface.

	/** Recursively visit each element in the schema. */
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

public:
	static constexpr const TCHAR* LogName = TEXT("WebAPI Schema");
};

template <typename ModelType>
ModelType* UWebAPISchema::AddModel(const TObjectPtr<UWebAPITypeInfo>& InTypeInfo)
{
	static_assert(TIsDerivedFrom<ModelType, UWebAPIModelBase>::Value, "Type is not derived from UWebAPIModelBase.");
	
	check(InTypeInfo);
	
	const TObjectPtr<UWebAPIModelBase>& AddedModelBase = Models.Add_GetRef(NewObject<ModelType>(this));

	const TObjectPtr<ModelType> AddedModel = Cast<ModelType>(AddedModelBase);
	AddedModel->Name = InTypeInfo; // @note: Name member of Param isn't a type

	FStringFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("ClassName"), ModelType::StaticClass()->GetName());
	FormatArgs.Add(TEXT("ModelName"), AddedModel->Name.ToString(true));

	const FName ModelObjectName = MakeUniqueObjectName(this, ModelType::StaticClass(), FName(FString::Format(TEXT("{ClassName}_{ModelName}"), FormatArgs)));
	AddedModel->Rename(*ModelObjectName.ToString(), AddedModel->GetOuter());

	if(InTypeInfo)
	{
		const FText LogMessage = FText::FormatNamed(
		NSLOCTEXT("WebAPISchema", "AddedModelWithName", "Added a new Model \"{ModelName}\"."),
			TEXT("ModelName"), FText::FromString(AddedModel->Name.ToString(true)));
		GetMessageLog()->LogInfo(LogMessage, UWebAPISchema::LogName);
	}
	else
	{
		GetMessageLog()->LogInfo(NSLOCTEXT("WebAPISchema", "AddedModel", "Added a new (unnamed) Model."), UWebAPISchema::LogName);
	}
	
	return AddedModel;
}
