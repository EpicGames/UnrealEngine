// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/WebAPISchema.h"

#include "WebAPIService.generated.h"

class UWebAPIOperation;

/** A service generally contains a sub-section of the API containing operations related to a particular subject. */
UCLASS(MinimalAPI)
class UWebAPIService
	: public UWebAPIModelBase
{
	GENERATED_BODY()

public:
	/** Name of the Service. */
	UPROPERTY(EditAnywhere, Category = "Type")
	FWebAPITypeNameVariant Name;
	
	/** Can be disabled to skip generation of this Service and it's operations. */
	UPROPERTY(EditAnywhere, Category = "Type")
	bool bGenerate = true;

	/** Operations contained within the service. */
	UPROPERTY(EditAnywhere, Category = "Type")
	TArray<TObjectPtr<UWebAPIOperation>> Operations;

	//~ Begin IWebAPISchemaObjectInterface Interface.
	virtual void SetNamespace(const FString& InNamespace) override;
	virtual void Visit(TFunctionRef<void(IWebAPISchemaObjectInterface*&)> InVisitor) override;
	//~ End IWebAPISchemaObjectInterface Interface.

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
};
