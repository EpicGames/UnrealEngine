// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPISchema.h"
#include "UObject/Object.h"
#include "Styling/SlateColor.h"

#include "WebAPIType.generated.h"

class UGraphEditorSettings;
class UWebAPIModelBase;

/** Schema type allows same name but different Schema types, ie. both a service and model with the name "Pet". */
UENUM()
enum class EWebAPISchemaType : uint8
{
	Model,
	Service,
	Operation,

	// All items before this are considered top-level, all below are sub-items
	Property,
	Parameter,

	Unspecified = 128,
};

namespace UE
{
	namespace WebAPI
	{
		namespace WebAPISchemaType
		{
			const FString& ToString(EWebAPISchemaType SchemaType);	
		}			
	}
}

/** Holds information for an existing or pending type. */
UCLASS(BlueprintType)
class WEBAPIEDITOR_API UWebAPITypeInfo : public UObject
{
	GENERATED_BODY()

public:
	/** Type Name without prefix or namespace, ie. "Vector", not "FVector". */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString Name;

	/** Optional display name, different to the actual name, ie. "JsonObject" vs. "JsonObjectWrapper". */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString DisplayName;

	/** SchemaType to discern between ie. a Service and a Model with the same name. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	EWebAPISchemaType SchemaType = EWebAPISchemaType::Model;

	/** Flag specifying whether this type is used exclusively by a parent type (isn't shared). */
	UPROPERTY(VisibleAnywhere, Category = "Type", meta = (InlineEditConditionToggle))
	bool bIsNested = false;

	/** When the type is nested, this specifies who "owns" it. */
	UPROPERTY(VisibleAnywhere, Category = "Type", meta = (EditCondition = "bIsNested"))
	FWebAPITypeNameVariant ContainingType;

	/** Field name as sent to and received from the external API. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString JsonName;

	/** Json type. Should correspond with values in EJson. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FName JsonType = TEXT("Object");

	/** Optional sub-property to serialize, instead of the object itself. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString JsonPropertyToSerialize;

	/** Optional specifier for printf, ie. "s", "d". */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString PrintFormatSpecifier = TEXT("");

	/** Optional tokenized string expression to get the value for printf, ie. ToString({Property}) */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString PrintFormatExpression = TEXT("{Property}");

	/** Type Namespace, can be empty for built-in types. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString Namespace;

	/** Type Prefix, usually "F", "U", "A", or "E". */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString Prefix;

	/** Optional suffix, ie. "Parameter", "Item", etc. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString Suffix;

	/** Declaration type, ie. struct, enum. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FName DeclarationType = NAME_StructProperty;

	/** If this is false, the type is to be generated and should have a namespace. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	bool bIsBuiltinType = true;

	/** Default value as a string, if applicable. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString DefaultValue;

	/** Module dependencies for this type. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	TSet<FString> Modules = { TEXT("Core") };

	/** Relative include paths required when referencing this type. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	TSet<FString> IncludePaths = { TEXT("CoreMinimal.h") };

	/** Color for UI. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FSlateColor PinColor;

	/** Misc info for debug. */
	UPROPERTY(VisibleAnywhere, Category = "Type")
	FString DebugString;

	UWebAPITypeInfo() = default;

	bool IsEnum() const;

	/** Duplicate this TypeInfo to the provided TypeRegistry. */
	TObjectPtr<UWebAPITypeInfo> Duplicate(const TObjectPtr<class UWebAPITypeRegistry>& InTypeRegistry) const;

	/** Get the fully formatted name, with prefix and namespace. */
	FString ToString(bool bInJustName = false) const;

	/** Get the fully formatted name as a valid member name. Optionally provide a prefix for invalid member names (ie. that start with a number). */
	FString ToMemberName(const FString& InPrefix = {}) const;

	/** The name for UI display. */
	FString GetDisplayName() const;

	/** The json field name that the target API expects, rather than the UE friendly one. */
	FString GetJsonName() const;

	/** Default value as a string. */
	FString GetDefaultValue(bool bQualified = false) const;

	/** Set's the name and updates the underlying object id. */
	void SetName(const FString& InName);

	/** Set's this type as nested, using the provided Type as ContainingType. */
	void SetNested(const FWebAPITypeNameVariant& InNester);

	/** Get the (mutable) Model. Will return nullptr if not set. */
	UObject* GetModel() const;

	/** Set's the Model. */
	template <
		typename ModelType
		UE_REQUIRES(
			std::is_base_of_v<UObject, std::decay_t<ModelType>>
			&& std::is_base_of_v<IWebAPISchemaObjectInterface, std::decay_t<ModelType>>)>
	void SetModel(const ModelType* InModel)
	{
		Model = const_cast<UObject*>(static_cast<const UObject*>(InModel));
	}

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/** Comparisons. */
	friend bool operator==(const UWebAPITypeInfo& A, const UWebAPITypeInfo& B);

	friend bool operator!=(const UWebAPITypeInfo& A, const UWebAPITypeInfo& B)
	{
		return !(A == B);
	}

private:
	/** Associated model, if any. */
	UPROPERTY(VisibleAnywhere, Category = "Type", meta = (ObjectMustImplement = "/Script/WebAPIEditor.WebAPISchemaObjectInterface"))
	TSoftObjectPtr<UObject> Model;
};
