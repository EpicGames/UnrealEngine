// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIType.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/Object.h"

#include "WebAPITypeRegistry.generated.h"

/** Holds Type information for built-in types. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIStaticTypeRegistry final
	: public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/** Find or create the TypeInfo for the given built-in type. */
	const TObjectPtr<UWebAPITypeInfo>& GetOrMakeBuiltinType(const FString& InName, const FString& InJsonName, const FSlateColor& InPinColor, const FString& InPrefix = TEXT(""), FName InDeclarationType = NAME_None);
	
	/** Finds the TypeInfo with the give name. */
	const TObjectPtr<UWebAPITypeInfo>* FindBuiltinType(const FString& InName);

	/** Initializes and registers the built-in types. */
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	/** Used to retrieve PinColor. */
	static const UGraphEditorSettings* GetGraphEditorSettings();

	/** For JsonType, when there is a ToJson/FromJson overload for the given value. */
	inline static const FName ToFromJsonType = TEXT("ToFromHandler");

	/** For JsonObjectWrapper, manipulate the JsonObject directly (also JsonObjectToString, JsonObjectFromString). */
	inline static const FName ToFromJsonObject = TEXT("ToFromJsonObject");	

public:
	/** List of common built-in types, static for easy comparisons, etc. */	
	TObjectPtr<UWebAPITypeInfo> Nullptr;
	TObjectPtr<UWebAPITypeInfo> Auto;
	TObjectPtr<UWebAPITypeInfo> Void;

	TObjectPtr<UWebAPITypeInfo> String;
	TObjectPtr<UWebAPITypeInfo> Name;
	TObjectPtr<UWebAPITypeInfo> Text;
	TObjectPtr<UWebAPITypeInfo> Char;
	TObjectPtr<UWebAPITypeInfo> Boolean;
	TObjectPtr<UWebAPITypeInfo> Float;
	TObjectPtr<UWebAPITypeInfo> Double;
	TObjectPtr<UWebAPITypeInfo> Byte;
	TObjectPtr<UWebAPITypeInfo> Int16;
	TObjectPtr<UWebAPITypeInfo> Int32;
	TObjectPtr<UWebAPITypeInfo> Int64;

	TObjectPtr<UWebAPITypeInfo> FilePath;
	TObjectPtr<UWebAPITypeInfo> DateTime;

	/** Unnamed enum. */
	TObjectPtr<UWebAPITypeInfo> Enum;

	/** Unnamed object. */
	TObjectPtr<UWebAPITypeInfo> Object;

	/** Json object. */
	TObjectPtr<UWebAPITypeInfo> JsonObject;

	/** WebAPI Message Response object. */
	TObjectPtr<UWebAPITypeInfo> WebAPIMessageResponse;

	/** The name of a TypeInfo when it's unnamed/unknown. */
	static constexpr const TCHAR* UnnamedTypeName = TEXT("Unnamed");

private:
	/** Initializes common built-in types. */
	void InitializeTypes();

	/** Array of common built-in types. */
	UPROPERTY()
	TArray<TObjectPtr<UWebAPITypeInfo>> BuiltinTypes;
};

/** Holds Type information for a given Schema. */
UCLASS()
class WEBAPIEDITOR_API UWebAPITypeRegistry : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Find or create the TypeInfo for the given SchemaType and Name.
	 * @note Providing the schema type allows for a Model and Service (for example) to have the same name but not be considered the same type.
	 */
	TObjectPtr<UWebAPITypeInfo> GetOrMakeGeneratedType(const EWebAPISchemaType& InSchemaType, const FString& InName, const FString& InJsonName, const FString& InPrefix = TEXT(""), FName InDeclarationType = NAME_None);
	TObjectPtr<UWebAPITypeInfo> GetOrMakeGeneratedType(const EWebAPISchemaType& InSchemaType, const FString& InName, const FString& InJsonName, const TObjectPtr<const UWebAPITypeInfo>& InTemplateTypeInfo);

	/** Find and return the TypeInfo for the given SchemaType and Name. Returns nullptr if not found.  */
	const TObjectPtr<UWebAPITypeInfo>* FindGeneratedType(const EWebAPISchemaType& InSchemaType, const FString& InName);

	/** Returns the array containing all registered TypeInfo's for generation. */
	const TArray<TObjectPtr<UWebAPITypeInfo>>& GetGeneratedTypes() const;

	/** Returns the mutable array containing all registered TypeInfo's for generation. */
	TArray<TObjectPtr<UWebAPITypeInfo>>& GetMutableGeneratedTypes();

	/** Clears all stored type information. */
	void Clear();

	/** Checks for any unnamed types, logs a message and returns false if one or more unnamed still exist. */
	bool CheckAllNamed() const;

	/** Checks for and outputs any unnamed types, logs a message and returns false if one or more unnamed still exist. */
	bool CheckAllNamed(TArray<TObjectPtr<UWebAPITypeInfo>>& OutUnnamedTypes) const;

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

public:
	/** For use by the WebAPI Message Log. */
	static constexpr const TCHAR* LogName = TEXT("WebAPI TypeRegistry");

private:
	/** Array of TypeInfo objects to generate (no built-in types). */
	UPROPERTY()
	TArray<TObjectPtr<UWebAPITypeInfo>> GeneratedTypes;
};

namespace UE
{
	namespace WebAPI
	{
		/** Creates a unique FName for TypeInfo's based on the given parameters. */
		FName MakeTypeInfoName(UObject* InOuter, const EWebAPISchemaType& InSchemaType, const FString& InName);
	}
}
