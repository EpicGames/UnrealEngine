// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"
#include "EditorReimportHandler.h"
#include "Factories/Factory.h"

#include "WebAPIDefinitionFactory.generated.h"

class IWebAPIImportHandler;
class UWebAPIDefinition;

/** Base Factory for WebAPIDefinition. */
UCLASS(Abstract, BlueprintType)
class WEBAPIEDITOR_API UWebAPIDefinitionFactory
    : public UFactory
    , public FReimportHandler
{
    GENERATED_BODY()

public:
    UWebAPIDefinitionFactory();

	/** Check if the factory can import the given file (and it's contents). */
    virtual bool CanImportWebAPI(const FString& InFileName, const FString& InFileContents);

	/** Perform the import or re-import. */
	virtual TFuture<bool> ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents);

	/** Called after a successful import. */
	virtual TFuture<void> PostImportWebAPI(UWebAPIDefinition* InDefinition);

	/** Checks that the provided file extension (not the whole file name or path!) is valid for this factory. */
	virtual bool IsValidFileExtension(const FString& InFileExtension) const;

	//~ Begin UFactory Interface
    virtual bool FactoryCanImport(const FString& InFileName) override;
    virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled) override;
    //~ Begin UFactory Interface

    //~ Begin FReimportHandler Interface
    virtual bool CanReimport(UObject* InObj, TArray<FString>& OutFilenames) override;
    virtual void SetReimportPaths(UObject* InObj, const TArray<FString>& NewReimportPaths) override;
    virtual EReimportResult::Type Reimport(UObject* InObj) override;
    //~ End FReimportHandler Interface

protected:
    /** Could be on-disk or URL. */
    FString ReadFileContents(const FString& InFileName) const;

	/** Checks that the provided file contains a valid file extension. */
    virtual bool IsValidFile(const FString& InFilename) const;

private:
    static constexpr const TCHAR* LogName = TEXT("WebAPI Definition Factory");
};
