// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetToolsModule.h"
#include "WebAPIProvider.h"
#include "CodeGen/WebAPICodeGenerator.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

DECLARE_MULTICAST_DELEGATE(FOnWebAPIProvidersChanged);

class UWebAPIStaticTypeRegistry;

class IWebAPIEditorModuleInterface
    : public IModuleInterface
    , public IHasMenuExtensibility
    , public IHasToolBarExtensibility
{
public:
    /**
    * Singleton-like access to this module's interface.  This is just for convenience!
    * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
    *
    * @return Returns singleton instance, loading the module on demand if needed
    */
    static IWebAPIEditorModuleInterface& Get()
    {
        static const FName ModuleName = "WebAPIEditor";
        return FModuleManager::LoadModuleChecked<IWebAPIEditorModuleInterface>(ModuleName);
    }

    virtual EAssetTypeCategories::Type GetAssetCategory() const = 0;

	/** Adds and registers a WebAPI provider. */
	virtual bool AddProvider(FName InProviderName, TSharedRef<IWebAPIProviderInterface> InProvider) = 0;

	/** Removes a WebAPI provider. */
	virtual void RemoveProvider(FName InProviderName) = 0;

	/** Retrieve a registered provider by its name. */
	virtual TSharedPtr<IWebAPIProviderInterface> GetProvider(FName InProviderName) = 0;

	/** Retrieve all registered providers. */
	virtual bool GetProviders(TArray<TSharedRef<IWebAPIProviderInterface>>& OutProviders) = 0;

	/** Whenever a Provider is added or removed. */
	virtual FOnWebAPIProvidersChanged& OnProvidersChanged() = 0;

	/** Adds and registers a WebAPI code generator. */
	virtual bool AddCodeGenerator(FName InCodeGeneratorName, const TSubclassOf<UObject>& InCodeGenerator) = 0;

	/** Removes a WebAPI code generator. */
	virtual void RemoveCodeGenerator(FName InCodeGeneratorName) = 0;

	/** Retrieve a registered code generator by its name. */
	virtual TScriptInterface<IWebAPICodeGeneratorInterface> GetCodeGenerator(FName InCodeGeneratorName) = 0;

	/** Retrieve all registered code generators. */
	virtual bool GetCodeGenerators(TArray<TScriptInterface<IWebAPICodeGeneratorInterface>>& OutCodeGenerators) = 0;

	/** Returns the static type registry to retrieve metadata for built-in types. */
	virtual UWebAPIStaticTypeRegistry* GetStaticTypeRegistry() const = 0;

	DECLARE_EVENT_OneParam(FUVEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() = 0;
};
