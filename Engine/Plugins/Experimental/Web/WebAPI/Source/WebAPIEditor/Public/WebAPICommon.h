// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WebAPICommon.generated.h"

USTRUCT(BlueprintType)
struct WEBAPIEDITOR_API FWebAPITemplateString
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="String")
	FString String;

	FWebAPITemplateString() = default;

	explicit FWebAPITemplateString(FString&& InString)
		: String(MoveTemp(InString))
	{
	}

	FString PopulateTemplate(const FStringFormatNamedArguments& InArgs);

	/** Gets the validity of the template string. */
	bool IsValid();
	
	/** Gets the validity of the template string for the given named arguments. */
	bool IsValid(const FStringFormatNamedArguments& InArgs);
};
