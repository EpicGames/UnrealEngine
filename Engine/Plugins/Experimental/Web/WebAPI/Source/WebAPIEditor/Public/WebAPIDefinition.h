// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIProviderSettings.h"
#include "CodeGen/WebAPICodeGenerator.h"
#include "CodeGen/WebAPICodeGeneratorSettings.h"
#include "UObject/Object.h"

#include "WebAPIDefinition.generated.h"

struct FWebAPICodeGeneratorSettings;
class FWebAPIMessageLog;
class UWebAPISchema;

/** The asset containing various options and the schema itself. */
UCLASS(AutoCollapseCategories = "ImportSettings")
class WEBAPIEDITOR_API UWebAPIDefinition : public UObject
{
	GENERATED_BODY()

public:
    UWebAPIDefinition();
	virtual ~UWebAPIDefinition() override;

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this definition. */
	UPROPERTY(EditAnywhere, Instanced, NoClear, AdvancedDisplay, Category = "ImportSettings")
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif

	/** Retrieves the custom data for the given key, or creates it if it doesn't exist. */
	UObject* AddOrGetImportedDataCache(FName InKey, const TSubclassOf<UObject>& InDataCacheClass);

	template <typename DataCacheType>
	DataCacheType* AddOrGetImportedDataCache(FName InKey)
	{
		static_assert(TIsDerivedFrom<DataCacheType, UObject>::IsDerived, "DataCacheType should derive from UObject");
		return Cast<DataCacheType>(AddOrGetImportedDataCache(InKey, DataCacheType::StaticClass()));		
	}

	/** Get the user-specified Copyright Notice, or from the Project Settings if unspecified. */
	FString GetCopyrightNotice() const;

	/** Settings for the WebAPI provider. */
	const FWebAPIProviderSettings& GetProviderSettings() const;

	/** Settings for the WebAPI provider. */
	FWebAPIProviderSettings& GetProviderSettings();

	/** Settings for the code generation. */
	const FWebAPICodeGeneratorSettings& GetGeneratorSettings() const;

	/** Settings for the code generation. */
	FWebAPICodeGeneratorSettings& GetGeneratorSettings();

	UWebAPISchema* GetWebAPISchema();
	const UWebAPISchema* GetWebAPISchema() const;
	const TSharedPtr<FWebAPIMessageLog>& GetMessageLog() const;

#if WITH_EDITOR
	/** 
	 * @return		Returns Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass the rules. Returns NotValidated if no rules are set for this object.
	 */
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

private:
	friend class FWebAPIDefinitionDetailsCustomization;

	/** Settings for the WebAPI provider. */
	UPROPERTY(EditAnywhere, Category = "Provider", meta = (ShowOnlyInnerProperties, AllowPrivateAccess))
	FWebAPIProviderSettings ProviderSettings;
	
	/** Settings for code generation. */
	UPROPERTY(EditAnywhere, Category = "Generator", meta = (ShowOnlyInnerProperties, AllowPrivateAccess))
	FWebAPICodeGeneratorSettings GeneratorSettings;
	
#if WITH_EDITORONLY_DATA
	/** Optional data store, ie. schema file contents. */
	UPROPERTY()
	TMap<FName, TObjectPtr<UObject>> ImportedDataCache;
#endif

	/** The schema written by the provider. */
	UPROPERTY(Instanced)
	TObjectPtr<UWebAPISchema> WebAPISchema;

	/** MessageLog for this Definition. */
	TSharedPtr<FWebAPIMessageLog> MessageLog;

	void OnNamespaceChanged(const FString& InNewNamespace);
};
