// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

WEBAPIEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogWebAPIEditor, Log, All);
