// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/Object.h"

#include "WebAPIEditorSettings.generated.h"

/** */
UCLASS(BlueprintType, Config = Engine, DefaultConfig, meta = (DisplayName = "WebAPI"))
class WEBAPIEDITOR_API UWebAPIEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NoClear, Category = "Generator", meta = (MustImplement = "/Script/WebAPIEditor.WebAPICodeGeneratorInterface"))
	TSoftClassPtr<UObject> CodeGeneratorClass;
	
	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	TScriptInterface<class IWebAPICodeGeneratorInterface> GetGeneratorClass() const;
};
