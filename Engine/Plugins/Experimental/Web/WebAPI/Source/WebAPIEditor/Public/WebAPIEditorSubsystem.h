// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "WebAPIDefinition.h"

#include "WebAPIEditorSubsystem.generated.h"

class IWebAPIProviderInterface;
class IWebAPICodeGeneratorInterface;
class UWebAPIStaticTypeRegistry;

/** Common functionality and registry for WebAPI Editor. */
UCLASS()
class WEBAPIEDITOR_API UWebAPIEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UWebAPIEditorSubsystem();
	
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	/**
	* Before a WebAPI Definition is being (re)imported.
	*
	* @param Definition - The Definition being re-imported. Will be nullptr if it's a new import.
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWebAPIDefinitionPreImport, UWebAPIDefinition* /* Definition */);

	/**
	* After a WebAPI Definition is being (re)imported.
	*
	* @param bSuccessful - Indicates if the import was successful.
	* @param FactoryName - The (unique) name of the Factory that imported the Definition.
	* @param Definition - The Definition being imported.
	*/
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWebAPIDefinitionPostImport, bool /* bSuccessful */, FName /* FactoryName */, UWebAPIDefinition* /* Definition */);
	
	/**
	* After a new module or plugin is created.
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModuleCreated, FString /* ModuleName */);

	FOnWebAPIDefinitionPreImport& OnDefinitionPreImport();
	FOnWebAPIDefinitionPostImport& OnDefinitionPostImport();
	FOnModuleCreated& OnModuleCreated();

private:
	FOnWebAPIDefinitionPreImport OnDefinitionPreImportDelegate;
	FOnWebAPIDefinitionPostImport OnDefinitionPostImportDelegate;
	FOnModuleCreated OnModuleCreatedDelegate;
	
	void HandleDefinitionPreImport(UWebAPIDefinition* InDefinition);
	void HandleDefinitionPostImport(bool bInImportSuccessful, FName InFactoryName, UWebAPIDefinition* InDefinition);
};
