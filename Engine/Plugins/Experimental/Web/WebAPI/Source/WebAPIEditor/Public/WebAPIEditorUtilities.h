// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/BreakIterator.h"
#include "Templates/SubclassOf.h"

namespace UE::WebAPI
{
	class WEBAPIEDITOR_API FWebAPIStringUtilities
	{
	public:
		/** Get the singleton instance. */
		static TSharedRef<FWebAPIStringUtilities> Get();

		/** Converts the input string to PascalCase. The result may still contain characters illegal for a member name. */
		FString ToPascalCase(FStringView InString);

		/** Converts the input string to its initials. */
		FString ToInitials(FStringView InString);

		/** Makes the input string valid for use as a member name, ie. no reserved characters. Provide an optional prefix to apply to invalid names (ie. start with a number). */
		FString MakeValidMemberName(FStringView InString, const FString& InPrefix = {}) const;

	protected:
		// Allows MakeShared with private constructor
		friend class SharedPointerInternals::TIntrusiveReferenceController<FWebAPIStringUtilities, ESPMode::ThreadSafe>;
		
		TSharedPtr<IBreakIterator> BreakIterator;

		FWebAPIStringUtilities() = default;
	};

	class WEBAPIEDITOR_API FWebAPIEditorUtilities
	{
	public:
		/** Get the singleton instance. */
		static TSharedRef<FWebAPIEditorUtilities> Get();

		/** Get all required headers to reference the provided class. */
		bool GetHeadersForClass(const TSubclassOf<UObject>& InClass, TArray<FString>& OutHeaders);
		
		/** Get all required modules to reference the provided class. */
		bool GetModulesForClass(const TSubclassOf<UObject>& InClass, TArray<FString>& OutModules);

	protected:
		// Allows MakeShared with private constructor
		friend class SharedPointerInternals::TIntrusiveReferenceController<FWebAPIEditorUtilities, ESPMode::ThreadSafe>;

		FWebAPIEditorUtilities() = default;
	};;
};
