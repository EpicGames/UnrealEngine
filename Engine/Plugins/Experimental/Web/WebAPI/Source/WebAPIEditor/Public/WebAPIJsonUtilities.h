// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/TVariant.h"

namespace UE::Json
{
	// @note: Concepts are an easier way (than type traits) of checking if a call or specialization is valid
	namespace Concepts
	{
		/** Concept to check if T has NumericLimits, and isn't a bool. */
		struct CNumerical
		{
			template <typename T>
			auto Requires() -> decltype(
				std::is_same_v<typename TNumericLimits<T>::NumericType, T> &&
				!std::is_same_v<T, bool>);
		};

		/** Describes a type that provides a FromJson function. */
		struct CFromJsonable
		{
			template <typename T>
			auto Requires(T& Val, const TSharedRef<FJsonObject>& Arg) -> decltype(
				Val.FromJson(Arg)
			);
		};

		/** Describes a type the is derived from TMap. */
		struct CMap 
		{
			template <typename T>
			auto Requires(const TMap<typename T::KeyType, typename T::ValueType>* MapPtr, T* Ptr) -> decltype(
				MapPtr = Ptr
			);
		};
	}
	
	namespace TypeTraits
	{
		/** String-like value types. */
		template <typename ValueType, typename Enable = void>
		struct TIsStringLike
		{
			enum { Value = false };
		};

		template <typename ValueType>
		struct TIsStringLike<
				ValueType,
				typename TEnableIf<
					std::is_same_v<ValueType, FString> ||
					std::is_same_v<ValueType, FName> ||
					std::is_same_v<ValueType, FText>>::Type>
		{
			enum
			{
				Value = true
			};
		};

		/** Numeric value types. */
		template <typename ValueType>
		constexpr bool TIsNumeric_V = TModels_V<Concepts::CNumerical, std::decay_t<ValueType>>;

		/** ValueType has FromJson. */
		template <typename ValueType>
		constexpr bool THasFromJson_V = TModels_V<Concepts::CFromJsonable, std::decay_t<ValueType>>;

		/** ValueType is derived from TMap. */
		template <typename ValueType>
		constexpr bool TIsDerivedFromMap_V = TModels_V<Concepts::CMap, std::decay_t<ValueType>>;
	}

	/** Contains either an object constructed in place, or a reference to an object declared elsewhere. */
	template<class ObjectType>
	class TJsonReference
	{
	public:
		using ElementType = ObjectType;
		static constexpr ESPMode Mode = ESPMode::ThreadSafe;
			
		/** Constructs an empty Json Reference. */
		TJsonReference()
			: Object(nullptr)
		{
		}

		/** Sets the object path and flags it as a pending reference (to be resolved later). */
		void ResolveDeferred(const FString& InJsonPath)
		{
			bHasPendingResolve = true;
			Path = InJsonPath;
		}

		/** Attempts to resolve the reference, returns true if successful or already set. */
		bool TryResolve(const TSharedRef<FJsonObject>& InRootObject, const TFunctionRef<ObjectType(TSharedRef<FJsonObject>&)>& InSetter)
		{
			// Already attempted to resolve, return result or nullptr
			if(!bHasPendingResolve)
			{
				return Object.IsValid();
			}

			TArray<FString> PathSegments;
			GetPathSegments(PathSegments);
			PathSegments.RemoveAt(0); // Remove #/

			const TSharedPtr<FJsonObject> RootObject = InRootObject;
			const TSharedPtr<FJsonObject>* SubObject = &RootObject;
			for(const FString& PathSegment : PathSegments)
			{
				if(!(*SubObject)->TryGetObjectField(PathSegment, SubObject))
				{
					return false;
				}
			}

			Object = MoveTemp(InSetter(SubObject->ToSharedRef()));
			return Object.IsValid();
		}

		/** Returns the object referenced by this Json Reference, creating it if it doesn't exist. */
		ObjectType* Get()
		{
			if(!Object.IsValid())
			{
				Object = MakeShared<ObjectType, Mode>();
			}
			
			return Object.Get();
		}

		/** Returns the object referenced by this Json Reference, creating it if it doesn't exist. */
		TSharedPtr<ObjectType> GetShared()
		{
			if(!Object.IsValid())
			{
				Object = MakeShared<ObjectType, Mode>();
			}
			
			return Object;
		}

		/** Returns the object referenced by this Json Reference, or nullptr if it doesn't exist. */
		ObjectType* Get() const
		{
			return Object.Get();
		}

		/** Returns the object referenced by this Json Reference, or nullptr if it doesn't exist. */
		TSharedPtr<ObjectType> GetShared() const
		{
			return Object;
		}

		/** Set's the object if it's not already set. */
		bool Set(ObjectType&& InObject)
		{
			if(Object.IsValid())
			{
				return false;
			}
				
			bHasPendingResolve = false;
			Object = MoveTemp(InObject);
			return true;
		}

		/** Set's the object if it's not already set. */
		bool Set(TSharedPtr<ObjectType>&& InObject)
		{
			if(Object.IsValid())
			{
				return false;
			}
				
			bHasPendingResolve = false;
			Object = MoveTemp(InObject);
			return true;
		}

		/** Set's the object if it's not already set. */
		bool Set(const TSharedPtr<ObjectType>& InObject)
		{
			if(Object.IsValid())
			{
				return false;
			}
				
			bHasPendingResolve = false;
			Object = InObject;
			return true;
		}

		/** Checks if the underlying object has been set. */
		bool IsSet() const
		{
			return Object.IsValid();
		}

		/** Checks to see if this is actually pointing to an object. */
		explicit operator bool() const
		{
			return Object != nullptr;
		}

		/** Checks if the object is valid, or it's pending reference resolution. */
		bool IsValid() const { return Object != nullptr || bHasPendingResolve; }

		/** Dereferences the object*/
		ObjectType& operator*() const
		{
			check(IsValid());
			return *Object; 
		}

		/** Pointer to the underlying object. */
		ObjectType* operator->() const
		{
			check(IsValid());
			return Get();
		}

		const FString& GetPath() const
		{
			return Path;
		}

		/** Returns true if there were one or more segments. */
		bool GetPathSegments(TArray<FString>& OutSegments) const
		{
			return Path.ParseIntoArray(OutSegments, TEXT("/")) > 0;
		}

		FString GetLastPathSegment() const
		{
			TArray<FString> PathSegments;
			if(GetPathSegments(PathSegments))
			{
				return PathSegments.Last();
			}
			return TEXT("");
		}

	protected:
		/** The specified path to the actual object definition. */
		FString Path;

		/** Flag indicating this is a reference but not yet resolved. */
		bool bHasPendingResolve = false;

		/** Underling object pointer. */
		TSharedPtr<ObjectType> Object;
	};

	// Numeric
	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsNumeric_V<ValueType>, void>::Type
	As(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		OutValue = InJsonValue->AsNumber();
	}

	// String
	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsStringLike<ValueType>::Value, void>::Type
	As(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		OutValue = InJsonValue->AsString();
	}

	// Bool
	template <typename ValueType>
	constexpr typename TEnableIf<std::is_same_v<ValueType, bool>, void>::Type
	As(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		OutValue = InJsonValue->AsBool();
	}

	// Numeric
	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsNumeric_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsNumeric_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Enum
	template <typename EnumType>
	typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, EnumType& OutValue, const TMap<FString, EnumType>& InNameToValue = {});

	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, EnumType& OutValue, const TMap<FString, EnumType>& InNameToValue = {});

	// String
	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsStringLike<ValueType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsStringLike<ValueType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Bool
	template <typename ValueType>
	constexpr typename TEnableIf<std::is_same_v<ValueType, bool>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	constexpr typename TEnableIf<std::is_same_v<ValueType, bool>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Array
	template <typename ContainerType>
	typename TEnableIf<TIsTArray_V<ContainerType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ContainerType& OutValues);

	template <typename ContainerType>
	constexpr typename TEnableIf<TIsTArray_V<ContainerType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ContainerType& OutValues);

	// Map
	template <typename KeyType, typename ValueType>
	typename TEnableIf<TypeTraits::TIsStringLike<KeyType>::Value, bool>::Type // Key must be string
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TMap<KeyType, ValueType>& OutValues);

	template <typename KeyType, typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsStringLike<KeyType>::Value, bool>::Type // Key must be string
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TMap<KeyType, ValueType>& OutValues); // Key must be string

	template <typename MapType>
	typename TEnableIf<TIsTMap<MapType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, MapType& OutValues);

	template <typename MapType>
	constexpr typename TEnableIf<TIsTMap<MapType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, MapType& OutValues); // Key must be string
	
	// Object (with FromJson)
	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Variant
	template <typename... ValueTypes>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TVariant<ValueTypes...>& OutValue);

	template <typename... ValueTypes>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TVariant<ValueTypes...>& OutValue);

	// UniqueObj
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TUniqueObj<ValueType>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TUniqueObj<ValueType>& OutValue);

	// TJsonReference
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TJsonReference<ValueType>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TJsonReference<ValueType>& OutValue);

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Object (with FromJson)
	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// Object (without FromJson)
	template <typename ValueType>
	typename TEnableIf<
		!std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>> &&
		!std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>> &&
		!TIsTMap<ValueType>::Value &&
		!TIsTArray_V<ValueType> &&
		!TypeTraits::TIsStringLike<ValueType>::Value &&
		!TIsPODType<ValueType>::Value &&
		!TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	// TIsTMap<ContainerType>,
	template <typename ValueType>
	typename TEnableIf<
		!std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>> &&
		!std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>> &&
		!TIsTMap<ValueType>::Value &&
		!TIsTArray_V<ValueType> &&
		!TypeTraits::TIsStringLike<ValueType>::Value &&
		!TIsPODType<ValueType>::Value &&
		!TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// UniqueObj
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TUniqueObj<ValueType>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TUniqueObj<ValueType>& OutValue);

	// SharedPtr
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TSharedPtr<ValueType>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedPtr<ValueType>& OutValue);

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue);

	// SharedRef
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TSharedRef<ValueType>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedRef<ValueType>& OutValue);

	// Optional
	template <typename ValueType>
	constexpr typename TEnableIf<!TIsEnumClass<ValueType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<ValueType>& OutValue);

	template <typename ValueType>
	typename TEnableIf<!TIsEnumClass<ValueType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<ValueType>& OutValue);

	// Optional Enum
	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<EnumType>& OutValue, const TMap<FString, EnumType>& InNameToValue = {});

	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<EnumType>& OutValue, const TMap<FString, EnumType>& InNameToValue = {});

	// Optional UniqueObj
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<TUniqueObj<ValueType>>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<TUniqueObj<ValueType>>& OutValue);

	// Optional SharedRef
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<TSharedRef<ValueType>>& OutValue);

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<TSharedRef<ValueType>>& OutValue);

	// JsonObject
	template <typename ValueType, typename = typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<FJsonObject>>>::Type>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue);

	template <typename ValueType, typename = typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<FJsonObject>>>::Type>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedPtr<FJsonObject>& OutValue);

	// Variant
	struct FVariantValueVisitor
	{
		explicit FVariantValueVisitor(const TSharedPtr<FJsonValue>& InJsonValue)
			: JsonValue(InJsonValue)
		{
		}

		template <typename ValueType>
		bool operator()(ValueType& OutValue)
		{
			ValueType TempValue;
			if(UE::Json::TryGet(JsonValue, TempValue))
			{
				OutValue = MoveTemp(TempValue);
				return true;
			}

			return false;
		}

		const TSharedPtr<FJsonValue> JsonValue;
	};

	struct FVariantObjectVisitor
	{
		explicit FVariantObjectVisitor(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName)
			: JsonObject(InJsonObject)
			, FieldName(InFieldName)
		{
		}

		template <typename ValueType>
		bool operator()(ValueType& OutValue)
		{
			if(UE::Json::TryGetField(JsonObject, FieldName, OutValue))
			{
				return true;
			}

			return false;
		}

		const TSharedPtr<FJsonObject> JsonObject;
		const FString FieldName;
	};

	template <typename... ValueTypes>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TVariant<ValueTypes...>& OutValue);

	template <typename... ValueTypes>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TVariant<ValueTypes...>& OutValue);
}

#include "WebAPIJsonUtilities.inl"
