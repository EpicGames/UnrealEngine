// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebAPIEditorLog.h"
#include "Templates/IsUEnumClass.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"

namespace UE::Json
{
	// Numeric
	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsNumeric_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		return InJsonValue->TryGetNumber(OutValue);
	}

	template <typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsNumeric_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		return InJsonObject->TryGetNumberField(InFieldName, OutValue);
	}

	// Enum
	template <typename EnumType>
	typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGet(
		const TSharedPtr<FJsonValue>& InJsonValue,
		EnumType& OutValue,
		const TMap<FString, EnumType>& InNameToValue)
	{
		bool bResult = false;

		// Try bool first - TryGetNumber converts "false" to 0, etc. which is undesired for an enum
		if(InJsonValue->Type != EJson::Boolean)
		{
			// Try int only if bool conversion failed
			uint8 IntValue;
			bResult = InJsonValue->TryGetNumber(IntValue);
			if(bResult)
			{
				OutValue = StaticCast<EnumType>(IntValue);	
			}
			else
			{
				static const UEnum* TypeEnum = nullptr;

				if constexpr (TIsUEnumClass<EnumType>::Value)
				{
					TypeEnum = StaticEnum<EnumType>();
				}
				
				if(!TypeEnum && InNameToValue.IsEmpty())
				{
					// UE_LOGF(LogWebAPIEditor, Error, "Tried to convert enum from string, but no lookup was provided.");
					return bResult;
				}

				FString StrValue;
				bResult = InJsonValue->TryGetString(StrValue);
				if(bResult)
				{
					if(TypeEnum)
					{
						if(int64 EnumValue = TypeEnum->GetValueByNameString(StrValue);
							EnumValue != INDEX_NONE)
						{
							OutValue = StaticCast<EnumType>(EnumValue);
						}
					}
					else if(const EnumType* NameToValue = InNameToValue.Find(StrValue))
					{
						OutValue = *NameToValue;
					}
					else
					{
						// Got a valid string, but it wasn't in the NameToValue map
						bResult = false;
					}
				}
			}
		}
		return bResult;
	}

	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGetField(
		const TSharedPtr<FJsonObject>& InJsonObject,
		const FString& InFieldName,
		EnumType& OutValue,
		const TMap<FString, EnumType>& InNameToValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue, InNameToValue);
		}
		return false;
	}

	// String
	template <>
	inline bool TryGet<FString>(const TSharedPtr<FJsonValue>& InJsonValue, FString& OutValue)
	{
		return InJsonValue->TryGetString(OutValue);
	}

	template <>
	inline bool TryGetField<FString>(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, FString& OutValue)
	{
		return InJsonObject->TryGetStringField(InFieldName, OutValue);
	}

	// Text
	template <>
	inline bool TryGet<FText>(const TSharedPtr<FJsonValue>& InJsonValue, FText& OutValue)
	{
		FString StrValue;
		if(InJsonValue->TryGetString(StrValue))
		{
			OutValue = FText::FromString(StrValue);
			return true;
		}
		return false;
	}

	template <>
	inline bool TryGetField<FText>(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, FText& OutValue)
	{
		FString StrValue;
		if(InJsonObject->TryGetStringField(InFieldName, StrValue))
		{
			OutValue = FText::FromString(StrValue);
			return true;
		}
		return false;
	}

	// Bool
	template <typename ValueType>
	constexpr typename TEnableIf<std::is_same_v<ValueType, bool>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		return InJsonValue->TryGetBool(OutValue);
	}

	template <typename ValueType>
	constexpr typename TEnableIf<std::is_same_v<ValueType, bool>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		return InJsonObject->TryGetBoolField(InFieldName, OutValue);
	}

	// Array
	template <typename ContainerType>
	typename TEnableIf<TIsTArray_V<ContainerType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ContainerType& OutValues)
	{
		using ValueType = typename ContainerType::ElementType;
		const TArray<TSharedPtr<FJsonValue>>* Values = nullptr;
		if(InJsonValue->TryGetArray(Values))
		{
			bool bSuccess = true;
			if(Values->Num() > 0)
			{
				OutValues.Reserve(Values->Num());
				for(const TSharedPtr<FJsonValue>& JsonValue : *Values)
				{
					ValueType& Value = OutValues.Emplace_GetRef();
					bSuccess &= TryGet(JsonValue, Value);
				}
			}
			return bSuccess;
		}
		return false;
	}

	template <typename ContainerType>
	constexpr typename TEnableIf<TIsTArray_V<ContainerType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ContainerType& OutValues)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			using ValueType = typename ContainerType::ElementType;
			return TryGet<ContainerType>(JsonField, OutValues);
		}
		return false;
	}

	// Map
	template <typename KeyType, typename ValueType>
	typename TEnableIf<TypeTraits::TIsStringLike<KeyType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TMap<KeyType, ValueType>& OutValues)
	{
		const TSharedPtr<FJsonObject>* JsonObject = nullptr;
		if(InJsonValue->TryGetObject(JsonObject))
		{
			OutValues.Reserve(JsonObject->Get()->Values.Num());
			for(TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue, ESPMode::ThreadSafe>>& FieldValuePair : JsonObject->Get()->Values)
			{
				ValueType& ItemValue = OutValues.Emplace(MoveTemp(FieldValuePair.Key));
				TryGet(FieldValuePair.Value, ItemValue);
			}
			return true;
		}
		return false;
	}

	template <typename KeyType, typename ValueType>
	constexpr typename TEnableIf<TypeTraits::TIsStringLike<KeyType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TMap<KeyType, ValueType>& OutValues)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValues);
		}
		return false;
	}

	template <typename MapType>
	typename TEnableIf<TIsTMap<MapType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, MapType& OutValues)
	{
		using KeyType = typename MapType::KeyType;
		using ValueType = typename MapType::ValueType;

		return TryGet<KeyType, ValueType>(InJsonValue, OutValues);
	}

	template <typename MapType>
	constexpr typename TEnableIf<TIsTMap<MapType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, MapType& OutValues)
	{
		using KeyType = typename MapType::KeyType;
		using ValueType = typename MapType::ValueType;

		return TryGetField<KeyType, ValueType>(InJsonObject, InFieldName, OutValues);
	}

	// Optional
	template <typename ValueType>
	constexpr typename TEnableIf<!TIsEnumClass<ValueType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<ValueType>& OutValue)
	{
		ValueType Value;
		if(TryGet<ValueType>(InJsonValue, Value))
		{
			OutValue = Value;
			return true;
		}
		return true;
	}

	template <typename ValueType>
	typename TEnableIf<!TIsEnumClass<ValueType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<ValueType>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			// Field found, so initialize TOptional
			if(!OutValue.IsSet())
			{
				OutValue = ValueType{};
			}
			return TryGet(JsonField, OutValue);
		}
		return true;
	}

	// Optional Enum
	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<EnumType>& OutValue, const TMap<FString, EnumType>& InNameToValue)
	{
		EnumType Value;
		if(TryGet<TDecay<EnumType>>(InJsonValue, Value, InNameToValue))
		{
			OutValue = Value;
			return true;
		}
		return true;
	}

	template <typename EnumType>
	constexpr typename TEnableIf<TIsEnumClass<EnumType>::Value, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<EnumType>& OutValue, const TMap<FString, EnumType>& InNameToValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			if(!OutValue.IsSet())
			{
				OutValue = EnumType{};
			}
				
			return TryGet(JsonField, OutValue.GetValue(), InNameToValue);
		}
		return true;
	}

	// Object (with FromJson)
	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		const TSharedPtr<FJsonObject>* Value;
		if(InJsonValue->TryGetObject(Value))
		{
			return OutValue.FromJson(Value->ToSharedRef());
		}
		return false;
	}

	template <typename ValueType>
	typename TEnableIf<
		!TypeTraits::TIsDerivedFromMap_V<ValueType> &&
		TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return false;
	}

	// Object (without FromJson)
	template <typename ValueType>
	typename TEnableIf<
		!std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>> &&
		!std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>> &&
		!TIsTMap<ValueType>::Value &&
		!TIsTArray_V<ValueType> &&
		!TypeTraits::TIsStringLike<ValueType>::Value &&
		!TIsPODType<ValueType>::Value &&
		!TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		static_assert(TypeTraits::THasFromJson_V<ValueType>, "ValueType must implement FromJson");
		return false;
	}

	template <typename ValueType>
	typename TEnableIf<
		!std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>> &&
		!std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>> &&
		!TIsTMap<ValueType>::Value &&
		!TIsTArray_V<ValueType> &&
		!TypeTraits::TIsStringLike<ValueType>::Value &&
		!TIsPODType<ValueType>::Value &&
		!TypeTraits::THasFromJson_V<ValueType>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		static_assert(TypeTraits::THasFromJson_V<ValueType>, "ValueType must implement FromJson");
		return false;
	}

	// UniqueObj
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TUniqueObj<ValueType>& OutValue)
	{
		return Json::TryGet(InJsonValue, OutValue.Get());
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TUniqueObj<ValueType>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue.Get());
		}
		return false;
	}

	// TJsonReference
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TJsonReference<ValueType>& OutValue)
	{
		const TSharedPtr<FJsonObject>* JsonObject = nullptr;
		if(InJsonValue->TryGetObject(JsonObject))
		{
			FString ReferencePath;
			if(JsonObject->Get()->TryGetStringField(TEXT("$ref"), ReferencePath))
			{
				OutValue.ResolveDeferred(ReferencePath);
				return true;
			}
		}
		return Json::TryGet(InJsonValue, *OutValue.Get());
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TJsonReference<ValueType>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return false;
	}

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		return TryGet<typename ValueType::ElementType>(InJsonValue, OutValue);
	}

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TJsonReference<typename ValueType::ElementType>>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		return TryGet<typename ValueType::ElementType>(InJsonObject, InFieldName, OutValue);
	}

	// SharedPtr
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue,	TSharedPtr<ValueType>& OutValue)
	{
		if(InJsonValue->IsNull())
		{
			return true;
		}
		
		// Initialize if invalid
		if(!OutValue.IsValid())
		{
			OutValue = MakeShared<ValueType>();
		}
			
		return Json::TryGet(InJsonValue, *OutValue.Get());
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedPtr<ValueType>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet<ValueType>(JsonField, OutValue);
		}
		return false;
	}

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>>, bool>::Type
	TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		return TryGet<typename ValueType::ElementType>(InJsonValue, OutValue);
	}

	template <typename ValueType>
	typename TEnableIf<std::is_same_v<ValueType, TSharedPtr<typename ValueType::ElementType>>, bool>::Type
	TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, ValueType& OutValue)
	{
		return TryGet<typename ValueType::ElementType>(InJsonObject, InFieldName, OutValue);
	}

	// SharedRef
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TSharedRef<ValueType>& OutValue)
	{
		return Json::TryGet(InJsonValue, OutValue.Get());
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedRef<ValueType>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return false;
	}

	// Optional UniqueObj
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TOptional<TUniqueObj<ValueType>>& OutValue)
	{
		if(OutValue.IsSet())
		{
			TUniqueObj<ValueType>& Value = OutValue.GetValue();
			if(TryGet(InJsonValue, OutValue.GetValue()))
			{
				OutValue = Value;
				return true;
			}
		}
		else
		{
			TUniqueObj<ValueType> Value;
			if(TryGet(InJsonValue, Value))
			{
				OutValue = Value;
				return true;
			}
		}	

		return true;
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<TUniqueObj<ValueType>>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return true;
	}

	// Optional SharedRef
	template <typename ValueType>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue,	TOptional<TSharedRef<ValueType>>& OutValue)
	{
		TSharedRef<ValueType> Value = MakeShared<ValueType>();
		if(TryGet(InJsonValue, Value))
		{
			OutValue = Value;
			return true;
		}
		return true;
	}

	template <typename ValueType>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TOptional<TSharedRef<ValueType>>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return true;
	}

	// JsonObject
	template <typename ValueType, typename>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, ValueType& OutValue)
	{
		const TSharedPtr<FJsonObject>* Value = nullptr;
		if(InJsonValue->TryGetObject(Value))
		{
			OutValue = *Value;
			return true;
		}
		return false;
	}

	template <typename ValueType, typename>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TSharedPtr<FJsonObject>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet<>(JsonField, OutValue);
		}
		return false;
	}

	template <size_t TypeIndex = 0, typename ...ValueTypes>
	bool TryParse(const TSharedPtr<FJsonValue>& InJsonValue, TVariant<ValueTypes...>& OutValue)
	{
		using VariantType = TVariant<ValueTypes...>;
		if constexpr (TypeIndex < TVariantSize_V<VariantType>)
		{
			using ValueType = typename TNthTypeFromParameterPack<TypeIndex, ValueTypes...>::Type;
			ValueType Value;
			if (UE::Json::TryGet(InJsonValue, Value))
			{
				OutValue.template Set<ValueType>(MoveTemp(Value));
				return true;
			}

			return TryParse<TypeIndex + 1, ValueTypes...>(InJsonValue, OutValue);
		}
		else
		{
			return false;
		}
	}

	// Variant
	template <typename... ValueTypes>
	bool TryGet(const TSharedPtr<FJsonValue>& InJsonValue, TVariant<ValueTypes...>& OutValue)
	{
		if(TryParse<0, ValueTypes...>(InJsonValue, OutValue))
		{
			return true;
		}
		return false;
	}

	template <typename... ValueTypes>
	bool TryGetField(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InFieldName, TVariant<ValueTypes...>& OutValue)
	{
		if(const TSharedPtr<FJsonValue> JsonField = InJsonObject->TryGetField(InFieldName))
		{
			return TryGet(JsonField, OutValue);
		}
		return false;
	}
}
