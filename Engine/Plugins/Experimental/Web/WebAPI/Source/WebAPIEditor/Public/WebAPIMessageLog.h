// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IMessageLogListing;
class SWidget;

/**
* Log verbosity level
*/
enum class EVerbosityLevel : uint8
{
	Log,
	Warning,
	Error
};

/** WebAPI Message Log, unique to each definition (when editing). */
class WEBAPIEDITOR_API FWebAPIMessageLog final : public TSharedFromThis<FWebAPIMessageLog>
{
public:
	FWebAPIMessageLog();
	
	/**
	* Log an information message
	* @param InMessage			The text message to log
	* @param InCallerName		Message type: provider or generator name
	*/
	void LogInfo(const FText& InMessage, const FString& InCallerName = TEXT(""));

	/**
	* Logs a warning message
	* @param InMessage			The text message to log
	* @param InCallerName		Message type: provider or generator name
	*/
	void LogWarning(const FText& InMessage, const FString& InCallerName = TEXT(""));

	/**
	* Logs an error message
	* @param InMessage			The text message to log
	* @param InCallerName		Message type: provider or generator name
	*/
	void LogError(const FText& InMessage, const FString& InCallerName = TEXT(""));

	/** Removes all messages from log. */
	void ClearLog() const;
	
	/** Log listing interface. */
	TSharedPtr<IMessageLogListing> GetMessageLogListing() const { return MessageLogListing; }

private:
	/**
	* Log a message
	* @param InMessage			The text message to log
	* @param InCallerName		Message type: provider or generator name
	*/
	template <EVerbosityLevel VerbosityLevel>
	void Log(const FText& InMessage, const FString& InCallerName = TEXT(""));
	
	/** Log listing interface. */
	TSharedPtr<IMessageLogListing> MessageLogListing;
};
