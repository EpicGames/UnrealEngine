// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"

class UWebAPIDefinition;

enum class EWebAPIConversionResult : uint8
{
	Failed = 0,
	Succeeded = 1,	
	FailedWithErrors = 2,
	FailedWithWarnings = 3
};

/** Interface for a provider of a WebAPI Schema */
class WEBAPIEDITOR_API IWebAPIProviderInterface
	: public TSharedFromThis<IWebAPIProviderInterface>
{
public:
	virtual ~IWebAPIProviderInterface() = default;

	/** Convert and modify the WebAPISchema for the given Definition. Returns true if successful. */
	virtual TFuture<EWebAPIConversionResult> ConvertToWebAPISchema(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition) = 0;
};
