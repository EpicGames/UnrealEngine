// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/WebAPISchema.h"

#include "WebAPIProviderSettings.generated.h"

/** Encapsulates settings for WebAPI providers. */
USTRUCT(BlueprintType)
struct WEBAPIEDITOR_API FWebAPIProviderSettings
{
	GENERATED_BODY()

	/** Makes a unique TypeName based on the provided Operation name. */
	FString MakeRequestTypeName(const FWebAPITypeNameVariant& InOperationName) const;

	/** Makes a unique TypeName based on the provided Operation name and response code. */
	FString MakeResponseTypeName(const FWebAPITypeNameVariant& InOperationName, const uint32& InResponseCode) const;

	/** Makes a unique TypeName based on the provided Model and Property names. */
	FString MakeNestedPropertyTypeName(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName) const;

	/** Makes a unique TypeName based on the provided Parameter name. */
	FString MakeParameterTypeName(const FWebAPINameVariant& InParameterName) const;

	/** Makes a unique TypeName based on the provided Service and Operations names. */
	FName MakeOperationObjectName(const TObjectPtr<UWebAPIService>& InService, const FWebAPITypeNameVariant& InOperationName) const;

	/** Return the name of an "unset" value for a generated enum. This allows optional enums that aren't otherwise supported as FProperties. */
	const FString& GetUnsetEnumValueName();

	/** Return the default name given to (single item) properties. */
	const FString& GetDefaultPropertyName();

	/** Return the default name given to array properties. */
	const FString& GetDefaultArrayPropertyName();

	/** Converts the given string to Pascal Case (aka. Upper Camel Case). */
	FString ToPascalCase(const FWebAPINameVariant& InString) const;

	/** Converts the given string to it's initials. */
	FString ToInitials(const FWebAPINameVariant& InString) const;

	/** Strips the provided string of invalid characters for a member name. Optionally provide a prefix for invalid member names (ie. that start with a number). */
	FString MakeValidMemberName(const FWebAPINameVariant& InString, const FString& InPrefix = {}) const;

	/** Returns a singularized version of the provided plural string. Currently only works for english words. */
	FString Singularize(const FString& InString) const;

	/** Returns a pluralized version of the provided string. Currently only works for english words. */
	FString Pluralize(const FString& InString) const;

public:
	/** Enable this option to allow the user to supply arbitrary json data to any request without parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Provider")
	bool bEnableArbitraryJsonPayloads = false;

private:
	// @note: Templates are made to ensure uniqueness, changing them will probably break this constraint.
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString RequestTypeNameTemplate = TEXT("{OperationName}_Request");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString ResponseTypeNameTemplate = TEXT("{OperationName}_Response_{ResponseCode}");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString NestedPropertyTypeNameTemplate = TEXT("{ModelName}{PropertyName}");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString ParameterTypeNameTemplate = TEXT("{ParameterName}");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString OperationObjectNameTemplate = TEXT("{ClassName}_{ServiceName}_{OperationName}");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString UnsetEnumValueName = TEXT("Unset");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString DefaultPropertyName = TEXT("Value");

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Provider")
	FString DefaultArrayPropertyName = TEXT("Values");

	/** Makes a unique TypeName based on the provided Operation Name. */
	FString PopulateTemplate(const FString& InTemplateString, const FStringFormatNamedArguments& InArgs) const;	
};
