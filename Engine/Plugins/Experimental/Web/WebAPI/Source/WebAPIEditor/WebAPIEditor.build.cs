// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebAPIEditor : ModuleRules
{
	public WebAPIEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"Core",
				"CoreUObject",
				"Engine",
				"GraphEditor",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"ApplicationCore",
				"AssetRegistry",
				"AssetTools",
				"Core",
				"CoreUObject",
				"CurveEditor",
				"DesktopPlatform",
				"DeveloperSettings",
				"EditorFramework",
				"EditorStyle",
				"EditorSubsystem",
				"EditorWidgets",
				"Engine",
				"EngineSettings",
				"GameProjectGeneration",
				"GraphEditor",
				"InputCore",
				"Json",
				"JsonUtilities",
				"MessageLog",
				"PluginBrowser",
				"Projects",
				"PropertyEditor",
				"Sequencer",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
				"WebAPI",
			});
    }
}
