// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPILiquidJSCodeGenerator.h"

#include "CodeGen/Dom/WebAPICodeGenBase.h"
#include "HttpModule.h"
#include "CodeGen/Dom/WebAPICodeGenClass.h"
#include "WebAPILiquidJSModule.h"
#include "CodeGen/Dom/WebAPICodeGenEnum.h"
#include "WebAPILiquidJSSettings.h"
#include "Algo/ForEach.h"
#include "Async/Async.h"
#include "CodeGen/Dom/WebAPICodeGenFile.h"
#include "CodeGen/Dom/WebAPICodeGenFunction.h"
#include "CodeGen/Dom/WebAPICodeGenOperation.h"
#include "CodeGen/Dom/WebAPICodeGenSettings.h"
#include "CodeGen/Dom/WebAPICodeGenStruct.h"
#include "Dom/WebAPIType.h"
#include "Dom/WebAPIOperation.h"
#include "Misc/FileHelper.h"
#include "Interfaces/IHttpRequest.h"
#include "Modules/ModuleManager.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "WebAPIDefinition.h"

#define LOCTEXT_NAMESPACE "WebAPILiquidJSCodeGenerator"

namespace UE
{
	namespace WebAPI
	{
		namespace Generator
		{
			namespace LiquidJS
			{
				/** Various functionality for converting the provided CodeGen objects to implementation-specific Json objects. */
				namespace Private
				{
					template <typename CodeGenType>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<CodeGenType>& InCodeGenObject);

					template <typename CodeGenType>
					TSharedPtr<FJsonObject> ToJson(const CodeGenType& InCodeGenObject);

					template <typename CodeGenType>
					void ToJson(const TArray<TSharedPtr<CodeGenType>>& InCodeGenObjects, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InCodeGenObjects.Num());
						for(const TSharedPtr<CodeGenType>& Element : InCodeGenObjects)
						{
							TSharedPtr<FJsonObject> ElementJson = ToJson(Element);
							OutJson.Add(MakeShared<FJsonValueObject>(ElementJson));
						}
					}

					template <>
					void ToJson<FWebAPICodeGenBase>(const TArray<TSharedPtr<FWebAPICodeGenBase>>& InCodeGenObjects, TArray<TSharedPtr<FJsonValue>>& OutJson);

					void ToJson(const TSet<FString>& InArray, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InArray.Num());
						for(const FString& Element : InArray)
						{
							TSharedPtr<FJsonValueString> ElementJson = MakeShared<FJsonValueString>(Element);
							OutJson.Add(MoveTemp(ElementJson));
						}
					}

					void ToJson(const TArray<FString>& InArray, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InArray.Num());
						for(const FString& Element : InArray)
						{
							TSharedPtr<FJsonValueString> ElementJson = MakeShared<FJsonValueString>(Element);
							OutJson.Add(MoveTemp(ElementJson));
						}
					}

					/** EnumValue Array ToJson */
					void ToJson(const TArray<FWebAPICodeGenEnumValue>& InArray, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InArray.Num());
						for(const FWebAPICodeGenEnumValue& Element : InArray)
						{
							TSharedPtr<FJsonObject> ElementJson = MakeShared<FJsonObject>();
							ElementJson->SetStringField(TEXT("name"), Element.Name);
							ElementJson->SetStringField(TEXT("displayName"), Element.DisplayName);
							ElementJson->SetStringField(TEXT("jsonName"), Element.JsonName);
							ElementJson->SetStringField(TEXT("description"), Element.Description);
							if(Element.IntegralValue >= 0)
							{
								ElementJson->SetNumberField(TEXT("explicitValue"), Element.IntegralValue);
							}							
							OutJson.Add(MakeShared<FJsonValueObject>(ElementJson));
						}
					}

					void ToJson(const TMap<FString, FString>& InMap, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InMap.Num());
						for(const TPair<FString, FString>& Element : InMap)
						{
							const TSharedPtr<FJsonObject> ElementJson = MakeShared<FJsonObject>();
							ElementJson->SetStringField(TEXT("key"), Element.Key);
							if(!Element.Value.IsEmpty())
							{
								ElementJson->SetStringField(TEXT("value"), Element.Value);
							}
							OutJson.Add(MakeShared<FJsonValueObject>(ElementJson));
						}
					}

					/** TypeNameVariant ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const FWebAPITypeNameVariant& InCodeGenObject)
					{
						check(InCodeGenObject.IsValid());

						TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
						if(InCodeGenObject.HasTypeInfo())
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject.TypeInfo->Name);
							JsonObject->SetStringField(TEXT("jsonName"), InCodeGenObject.TypeInfo->JsonName);
							JsonObject->SetStringField(TEXT("jsonType"), InCodeGenObject.TypeInfo->JsonType.ToString());
							JsonObject->SetStringField(TEXT("jsonPropertyToSerialize"), InCodeGenObject.TypeInfo->JsonPropertyToSerialize);
							JsonObject->SetStringField(TEXT("printFormatSpecifier"), InCodeGenObject.TypeInfo->PrintFormatSpecifier);
							JsonObject->SetStringField(TEXT("printFormatExpression"), InCodeGenObject.TypeInfo->PrintFormatExpression);
							JsonObject->SetStringField(TEXT("namespace"), InCodeGenObject.TypeInfo->Namespace);
							JsonObject->SetStringField(TEXT("prefix"), InCodeGenObject.TypeInfo->Prefix);
							JsonObject->SetStringField(TEXT("declarationType"), InCodeGenObject.TypeInfo->DeclarationType.ToString());
							JsonObject->SetBoolField(TEXT("isBuiltinType"), InCodeGenObject.TypeInfo->bIsBuiltinType);
						}

						return JsonObject;
					}

					/** NameVariant ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const FWebAPINameVariant& InCodeGenObject)
					{
						check(InCodeGenObject.IsValid());

						TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
						if(InCodeGenObject.HasNameInfo())
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject.NameInfo.Name);
							JsonObject->SetStringField(TEXT("jsonName"), InCodeGenObject.NameInfo.JsonName);
							JsonObject->SetStringField(TEXT("prefix"), InCodeGenObject.NameInfo.Prefix);
						}

						return JsonObject;
					}

					/** Base ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenBase>& InCodeGenObject)
					{
						check(InCodeGenObject);

						TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
						
						JsonObject->SetStringField(TEXT("description"), InCodeGenObject->Description);
						JsonObject->SetStringField(TEXT("module"), InCodeGenObject->Module);
						JsonObject->SetStringField(TEXT("namespace"), InCodeGenObject->Namespace);
						
						TArray<TSharedPtr<FJsonValue>> SpecifiersJson;
						ToJson(InCodeGenObject->Specifiers, SpecifiersJson);
						JsonObject->SetArrayField(TEXT("specifiers"), SpecifiersJson);

						TArray<TSharedPtr<FJsonValue>> MetadataJson;
						ToJson(InCodeGenObject->Metadata, MetadataJson);
						JsonObject->SetArrayField(TEXT("metadata"), MetadataJson);

						return JsonObject;
					}

					/** Property ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenProperty>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasNameInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						if(InCodeGenObject->Type.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> TypeJson = ToJson(InCodeGenObject->Type);
							JsonObject->SetObjectField(TEXT("type"), TypeJson);

							FString DefaultValue = InCodeGenObject->Type.TypeInfo->DefaultValue;
							// Wrap in array initializer, if it's an array
							if(InCodeGenObject->bIsArray)
							{
								DefaultValue = TEXT("{ }");
							}
							JsonObject->SetStringField(TEXT("defaultValue"), DefaultValue);
						}
						else
						{
							JsonObject->SetStringField(TEXT("type"), InCodeGenObject->Type.ToString(true));
						}

						if(!InCodeGenObject->DefaultValue.IsEmpty())
						{
							JsonObject->SetStringField(TEXT("defaultValue"), InCodeGenObject->DefaultValue);
						}

						JsonObject->SetBoolField(TEXT("isRequired"), InCodeGenObject->bIsRequired);
						JsonObject->SetBoolField(TEXT("isArray"), InCodeGenObject->bIsArray);
						JsonObject->SetBoolField(TEXT("isMixin"), InCodeGenObject->bIsMixin);

						return JsonObject;
					}

					/** Enum ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenEnum>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));
						
						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						TArray<TSharedPtr<FJsonValue>> ValuesJson;
						ToJson(InCodeGenObject->Values, ValuesJson);
						JsonObject->SetArrayField(TEXT("values"), ValuesJson);

						return JsonObject;
					}

					/** Struct ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenStruct>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						TArray<TSharedPtr<FJsonValue>> PropertiesJson;
						ToJson(InCodeGenObject->Properties, PropertiesJson);
						JsonObject->SetArrayField(TEXT("properties"), PropertiesJson);

						return JsonObject;
					}

					/** Class ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenClass>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenStruct>(StaticCastSharedPtr<FWebAPICodeGenStruct>(InCodeGenObject));

						TArray<TSharedPtr<FJsonValue>> FunctionsJson;
						ToJson(InCodeGenObject->Functions, FunctionsJson);
						JsonObject->SetArrayField(TEXT("functions"), FunctionsJson);

						return JsonObject;
					}

					/** FunctionParameter ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenFunctionParameter>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenProperty>(StaticCastSharedPtr<FWebAPICodeGenProperty>(InCodeGenObject));
						
						return JsonObject;
					}

					/** Function ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenFunction>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasNameInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						JsonObject->SetBoolField(TEXT("isOverride"), InCodeGenObject->bIsOverride);
						JsonObject->SetBoolField(TEXT("isConst"), InCodeGenObject->bIsOverride);

						if(InCodeGenObject->ReturnType.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> TypeJson = ToJson(InCodeGenObject->ReturnType);
							JsonObject->SetObjectField(TEXT("returnType"), TypeJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("returnType"), InCodeGenObject->ReturnType.ToString(true));
						}
						
						JsonObject->SetBoolField(TEXT("isReturnTypeConst"), InCodeGenObject->bIsConstReturnType);
						JsonObject->SetStringField(TEXT("body"), InCodeGenObject->Body);

						TArray<TSharedPtr<FJsonValue>> ParameterJson;
						ToJson(InCodeGenObject->Parameters, ParameterJson);
						JsonObject->SetArrayField(TEXT("parameters"), ParameterJson);

						return JsonObject;
					}

					/** OperationParameter ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenOperationParameter>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenProperty>(StaticCastSharedPtr<FWebAPICodeGenProperty>(InCodeGenObject));

						JsonObject->SetStringField(TEXT("storage"), UE::WebAPI::WebAPIParameterStorage::ToString(InCodeGenObject->Storage));
						JsonObject->SetStringField(TEXT("mediaType"), InCodeGenObject->MediaType);

						return JsonObject;
					}

					/** OperationRequest ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenOperationRequest>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						TArray<TSharedPtr<FJsonValue>> ParametersJson;
						ToJson(InCodeGenObject->Parameters, ParametersJson);
						JsonObject->SetArrayField(TEXT("properties"), ParametersJson); // encode as properties

						return JsonObject;
					}

					/** OperationResponse ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenOperationResponse>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenStruct>(StaticCastSharedPtr<FWebAPICodeGenStruct>(InCodeGenObject));

						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						JsonObject->SetNumberField(TEXT("responseCode"), InCodeGenObject->ResponseCode);
						JsonObject->SetStringField(TEXT("message"), InCodeGenObject->Message);

						checkf(InCodeGenObject->Base.IsValid(), TEXT("OperationResponse should have a base type set."));

						const TSharedPtr<FJsonObject> BaseJson = ToJson(InCodeGenObject->Base);
						JsonObject->SetObjectField(TEXT("base"), BaseJson);

						return JsonObject;
					}

					/** Operation ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenOperation>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						JsonObject->SetStringField(TEXT("path"), InCodeGenObject->Path);
						JsonObject->SetStringField(TEXT("service"), InCodeGenObject->ServiceName);
						JsonObject->SetStringField(TEXT("verb"), InCodeGenObject->Verb);

						TArray<TSharedPtr<FJsonValue>> RequestsJson;
						ToJson(InCodeGenObject->Requests, RequestsJson);
						JsonObject->SetArrayField(TEXT("requests"), RequestsJson);

						TArray<TSharedPtr<FJsonValue>> ResponsesJson;
						ToJson(InCodeGenObject->Responses, ResponsesJson);
						JsonObject->SetArrayField(TEXT("responses"), ResponsesJson);

						JsonObject->SetStringField(TEXT("settingsTypeName"), InCodeGenObject->SettingsTypeName);

						return JsonObject;
					 }

					/** Settings ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenSettings>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TSharedPtr<FJsonObject> JsonObject = ToJson<FWebAPICodeGenBase>(StaticCastSharedPtr<FWebAPICodeGenBase>(InCodeGenObject));

						if(InCodeGenObject->Name.HasTypeInfo())
						{
							const TSharedPtr<FJsonObject> NameJson = ToJson(InCodeGenObject->Name);
							JsonObject->SetObjectField(TEXT("name"), NameJson);
						}
						else
						{
							JsonObject->SetStringField(TEXT("name"), InCodeGenObject->Name.ToString(true));
						}

						if(InCodeGenObject->ParentType.IsValid())
						{
							JsonObject->SetStringField(TEXT("base"), InCodeGenObject->ParentType->GetName());	
						}
						JsonObject->SetStringField(TEXT("host"), InCodeGenObject->Host);
						JsonObject->SetStringField(TEXT("baseUrl"), InCodeGenObject->BaseUrl);
						JsonObject->SetStringField(TEXT("userAgent"), InCodeGenObject->UserAgent);
						JsonObject->SetStringField(TEXT("dateTimeFormat"), InCodeGenObject->DateTimeFormat);

						TArray<TSharedPtr<FJsonValue>> SchemesJson;
						ToJson(InCodeGenObject->Schemes, SchemesJson);
						JsonObject->SetArrayField(TEXT("schemes"), SchemesJson);

						return JsonObject;
					}

					/** File ToJson */
					template <>
					TSharedPtr<FJsonObject> ToJson(const TSharedPtr<FWebAPICodeGenFile>& InCodeGenObject)
					{
						check(InCodeGenObject);

						const TFunction<FString(FWebAPITypeNameVariant)> MakeForwardDeclarationString =
							[](const FWebAPITypeNameVariant& InTypeName)
							{
								const FString ObjectType = InTypeName.TypeInfo->IsEnum()
															? TEXT("enum class")
															: InTypeName.TypeInfo->Prefix == TEXT("F")
																? TEXT("struct")
																: TEXT("class");

								const FString Suffix = InTypeName.TypeInfo->IsEnum() ? TEXT(" : uint8") : TEXT("");								
								
								return FString::Printf(TEXT("%s %s%s"), *ObjectType, *InTypeName.ToString(), *Suffix);
							};

						const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

						JsonObject->SetStringField(TEXT("baseFilePath"), InCodeGenObject->BaseFilePath);
						JsonObject->SetStringField(TEXT("relativeFilePath"), InCodeGenObject->RelativeFilePath);
						JsonObject->SetStringField(TEXT("fileName"), InCodeGenObject->FileName);
						JsonObject->SetStringField(TEXT("fileType"), InCodeGenObject->FileType);
						JsonObject->SetStringField(TEXT("namespace"), InCodeGenObject->Namespace);
						JsonObject->SetStringField(TEXT("module"), InCodeGenObject->Module);
						JsonObject->SetStringField(TEXT("copyrightNotice"), InCodeGenObject->CopyrightNotice);

						TArray<TSharedPtr<FJsonValue>> IncludePathJson;
						ToJson(InCodeGenObject->IncludePaths, IncludePathJson);
						JsonObject->SetArrayField(TEXT("includePaths"), IncludePathJson);

						TArray<FString> ForwardDeclarations;

						// Find and transform Enums
						TArray<TSharedPtr<FJsonValue>> EnumsJson;
						Algo::TransformIf(
							InCodeGenObject->SubItems,
							EnumsJson,
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								return InItem->GetTypeName() == TEXT("Enum");
							},
							[&](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								const TSharedPtr<FWebAPICodeGenEnum> Enum = StaticCastSharedPtr<FWebAPICodeGenEnum>(InItem);
								ForwardDeclarations.Add(MakeForwardDeclarationString(Enum->Name));
								
								TSharedPtr<FJsonObject> EnumJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(Enum);
								return MakeShared<FJsonValueObject>(EnumJson);
							});

						// Find and transform Structs
						TArray<TSharedPtr<FJsonValue>> StructsJson;
						Algo::TransformIf(
							InCodeGenObject->SubItems,
							StructsJson,
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								return InItem->GetTypeName() == TEXT("Struct");
							},
							[&](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								const TSharedPtr<FWebAPICodeGenStruct> Struct = StaticCastSharedPtr<FWebAPICodeGenStruct>(InItem);
								ForwardDeclarations.Add(MakeForwardDeclarationString(Struct->Name));

								TSharedPtr<FJsonObject> StructJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(Struct);								
								return MakeShared<FJsonValueObject>(StructJson);
							});

						// Find and transform Classes
						TArray<TSharedPtr<FJsonValue>> ClassesJson;
						Algo::TransformIf(
							InCodeGenObject->SubItems,
							ClassesJson,
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								return InItem->GetTypeName() == TEXT("Class");
							},
							[&](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								const TSharedPtr<FWebAPICodeGenClass> Class = StaticCastSharedPtr<FWebAPICodeGenClass>(InItem);
								ForwardDeclarations.Add(MakeForwardDeclarationString(Class->Name));
								
								TSharedPtr<FJsonObject> ClassJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(Class);
								return MakeShared<FJsonValueObject>(ClassJson);
							});

						// Find and transform Operations
						TArray<TSharedPtr<FJsonValue>> OperationsJson;
						TArray<TSharedPtr<FJsonValue>> RequestsJson;
						TArray<TSharedPtr<FJsonValue>> ResponsesJson;
						Algo::TransformIf(
							InCodeGenObject->SubItems,
							OperationsJson,
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								return InItem->GetTypeName() == TEXT("Operation");
							},
							[&](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								const TSharedPtr<FWebAPICodeGenOperation> Operation = StaticCastSharedPtr<FWebAPICodeGenOperation>(InItem);
								TSharedPtr<FJsonObject> OperationJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(Operation);

								ForwardDeclarations.Add(MakeForwardDeclarationString(Operation->Name));

								RequestsJson.Append(OperationJson->GetArrayField(TEXT("requests")));
								ResponsesJson.Append(OperationJson->GetArrayField(TEXT("responses")));
								
								return MakeShared<FJsonValueObject>(OperationJson);
							});

						// Find and transform Settings
						TArray<TSharedPtr<FJsonObject>> SettingsJsonArray;
						Algo::TransformIf(
							InCodeGenObject->SubItems,
							SettingsJsonArray,
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								return InItem->GetTypeName() == TEXT("Settings");
							},
							[](const TSharedPtr<FWebAPICodeGenBase>& InItem)
							{
								const TSharedPtr<FWebAPICodeGenSettings> Settings = StaticCastSharedPtr<FWebAPICodeGenSettings>(InItem);
								TSharedPtr<FJsonObject> SettingsJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(Settings);
								return SettingsJson;
							});

						// Set here as Operations can add additional structs/enums
						JsonObject->SetArrayField(TEXT("enums"), EnumsJson);
						JsonObject->SetArrayField(TEXT("structs"), StructsJson);
						JsonObject->SetArrayField(TEXT("classes"), ClassesJson);
						JsonObject->SetArrayField(TEXT("operations"), OperationsJson);
						JsonObject->SetArrayField(TEXT("requests"), RequestsJson);
						JsonObject->SetArrayField(TEXT("responses"), ResponsesJson);

						TArray<TSharedPtr<FJsonValue>> ForwardDeclarationsJson;
						ToJson(ForwardDeclarations, ForwardDeclarationsJson);
						JsonObject->SetArrayField(TEXT("forwardDeclarations"), ForwardDeclarationsJson);

						if(!SettingsJsonArray.IsEmpty())
						{
							JsonObject->SetObjectField(TEXT("settings"), SettingsJsonArray.Last());							
						}

						return JsonObject;
					}

					template <>
					void ToJson<FWebAPICodeGenBase>(const TArray<TSharedPtr<FWebAPICodeGenBase>>& InCodeGenObjects, TArray<TSharedPtr<FJsonValue>>& OutJson)
					{
						OutJson.Reserve(InCodeGenObjects.Num());
						for(const TSharedPtr<FWebAPICodeGenBase>& CodeGenObject : InCodeGenObjects)
						{
							TSharedPtr<FJsonObject> ElementJson = MakeShared<FJsonObject>();
							const FName& CodeGenObjectType = CodeGenObject->GetTypeName();
							if(CodeGenObjectType == TEXT("Enum"))
							{
								ElementJson = ToJson(StaticCastSharedPtr<FWebAPICodeGenEnum>(CodeGenObject));
							}
							else if(CodeGenObjectType == TEXT("Struct"))
							{
								ElementJson = ToJson(StaticCastSharedPtr<FWebAPICodeGenStruct>(CodeGenObject));
							}
							else if(CodeGenObjectType == TEXT("Operation"))
							{
								ElementJson = ToJson(StaticCastSharedPtr<FWebAPICodeGenOperation>(CodeGenObject));
							}
							else if(CodeGenObjectType == TEXT("Class"))
							{
								ElementJson = ToJson(StaticCastSharedPtr<FWebAPICodeGenClass>(CodeGenObject));
							}
							OutJson.Add(MakeShared<FJsonValueObject>(ElementJson));
						}
					}

					/** Runs the given function on the GameThread, returning a Future. */
					template<typename ResultType>
					static TFuture<ResultType> ExecuteOnGameThread(const TFunction<ResultType()>& Function)
					{
						TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShared<TPromise<ResultType>>();
						TFunction<void()> PromiseKeeper =
							[Promise, Function]
						{
								Promise->SetValue(Function());
							};
						
						if (!IsInGameThread())
						{
							AsyncTask(ENamedThreads::GameThread, MoveTemp(PromiseKeeper));
						}
						else
						{
							PromiseKeeper();
						}
						return Promise->GetFuture();
					}
				}
			}
		}
	}
}

TFuture<bool> UWebAPILiquidJSCodeGenerator::IsAvailable()
{
	const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShared<TPromise<bool>, ESPMode::ThreadSafe>();

	FHttpModule& HttpModule = FHttpModule::Get();

	const FString URL = GetMutableDefault<UWebAPILiquidJSSettings>()->GetServiceUrl(TEXT("/api/ping"));

	const TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Request = HttpModule.CreateRequest();
	Request->SetVerb(TEXT("GET"));
	Request->SetURL(URL);
	Request->SetHeader(TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent"));
	Request->SetHeader(TEXT("Accept"), TEXT("application/json"));
	Request->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));
	Request->SetTimeout(10.0f); // 10 second timeout

	std::atomic<bool> bHasRetried = false;

	Request->OnProcessRequestComplete().BindLambda(
	[Promise, &bHasRetried, Request](FHttpRequestPtr, FHttpResponsePtr InResponse, bool bInWasSuccessful)
	{
		if(bInWasSuccessful && EHttpResponseCodes::IsOk(InResponse->GetResponseCode()))
		{
			Promise->SetValue(true);
		}
		else
		{
			if(!bHasRetried.load(std::memory_order_relaxed))
			{
				// Try starting
				bool bRetryResult = FModuleManager::Get().GetModulePtr<FWebAPILiquidJSModule>(TEXT("WebAPILiquidJS"))->TryStartWebApp();
				bHasRetried.store(true, std::memory_order_relaxed);
				Request->ProcessRequest();
				return;
			}

			Promise->SetValue(false);
		}
	});

	Request->ProcessRequest();

	return Promise->GetFuture();
}

TFuture<EWebAPIGenerationResult> UWebAPILiquidJSCodeGenerator::GenerateFile(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenFile>& InFile)
{
	check(InDefinition.IsValid());
	
	const TSharedRef<TPromise<EWebAPIGenerationResult>, ESPMode::ThreadSafe> Promise = MakeShared<TPromise<EWebAPIGenerationResult>, ESPMode::ThreadSafe>();

	FHttpModule& HttpModule = FHttpModule::Get();

	const FString FileType = InFile->FileType.EndsWith(TEXT("h"))
							? TEXT("decl")
							: TEXT("defn");

	const FString URL = GetDefault<UWebAPILiquidJSSettings>()->GetServiceUrl(TEXT("api/gen/") + FileType);

	const TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Request = HttpModule.CreateRequest();
	Request->SetVerb(TEXT("POST"));
	Request->SetURL(URL);
	Request->SetHeader(TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	Request->SetHeader(TEXT("Accept"), TEXT("application/json"));
	Request->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));

	const TSharedPtr<FJsonObject> FileJson = UE::WebAPI::Generator::LiquidJS::Private::ToJson(InFile);
#if WITH_WEBAPI_DEBUG
	bool bWriteResult = InDefinition->GetGeneratorSettings().bWriteResult;
#else
	bool bWriteResult = true;
#endif

	FString JsonString;
	const TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonString);
	FJsonSerializer::Serialize(FileJson.ToSharedRef(), Writer);

	Request->SetContentAsString(JsonString);

	Request->OnProcessRequestComplete().BindLambda(
	[Promise, InFile, bWriteResult, InDefinition](FHttpRequestPtr, FHttpResponsePtr InResponse, bool bInWasSuccessful)
	{
		if(bInWasSuccessful && InResponse->GetResponseCode() == 200)
		{
			const FString GeneratedCodeString = InResponse->GetContentAsString();

			// Write the result to the original schema objects (allows for visualization and debug of generated code)
 			Algo::ForEach(InFile->SchemaObjects, [&GeneratedCodeString](const TWeakObjectPtr<UObject>& InSchemaObject)
			{
				if(InSchemaObject.IsValid())
				{
					const TScriptInterface<IWebAPISchemaObjectInterface> AsInterface = InSchemaObject.Get();
					AsInterface.GetInterface()->AppendCodeText(GeneratedCodeString);
				}
			});
			
			if(bWriteResult)
			{
				FFileHelper::SaveStringToFile(GeneratedCodeString, *InFile->GetFullPath());	
			}
			
			Promise->SetValue(EWebAPIGenerationResult::Succeeded);
		}
		else
		{
			const FString JsonString = InResponse->GetContentAsString();				
			const TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonString);

			FFormatNamedArguments Args;
			Args.Add(TEXT("Code"), InResponse->GetResponseCode());
			Args.Add(TEXT("Url"), FText::FromString(InResponse->GetURL()));
			
			TSharedPtr<FJsonObject> JsonObject;
			if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
			{
				FString ErrorName;
				JsonObject->TryGetStringField(TEXT("errorName"), ErrorName);

				FString Message;
				JsonObject->TryGetStringField(TEXT("message"), Message);

				FString Stack;
				JsonObject->TryGetStringField(TEXT("stack"), Stack);

				Args.Add(TEXT("Error"), FText::FromString(ErrorName));
				Args.Add(TEXT("Message"), FText::FromString(Message));
				Args.Add(TEXT("StackTrace"), FText::FromString(Stack));
				InDefinition->GetMessageLog()->LogError(FText::Format(LOCTEXT("HttpErrorWithMessage", "HTTP Response Code: {Code}\nError: {Error}\nMessage: {Message}\nStackTrace: {StackTrace}"), Args), UWebAPILiquidJSCodeGenerator::LogName);
			}
			else
			{
				InDefinition->GetMessageLog()->LogError(FText::Format(LOCTEXT("HttpError", "HTTP Response Code: {Code}\nUrl: {Url}"), Args), UWebAPILiquidJSCodeGenerator::LogName);
			}

			Promise->SetValue(EWebAPIGenerationResult::Failed);
		}
	});

	Request->ProcessRequest();

	return Promise->GetFuture();
}

#undef LOCTEXT_NAMESPACE
