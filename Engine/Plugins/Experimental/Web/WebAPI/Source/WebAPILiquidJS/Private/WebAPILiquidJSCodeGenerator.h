// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeGen/WebAPICodeGenerator.h"

#include "WebAPILiquidJSCodeGenerator.generated.h"

/**
 * 
 */
UCLASS()
class WEBAPILIQUIDJS_API UWebAPILiquidJSCodeGenerator
	: public UWebAPICodeGeneratorBase
{
	GENERATED_BODY()
	
public:
	virtual TFuture<bool> IsAvailable() override;
	virtual TFuture<EWebAPIGenerationResult> GenerateFile(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition, const TSharedPtr<FWebAPICodeGenFile>& InFile) override;
};
