// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPILiquidJSModule.h"

#include "IWebAPIEditorModule.h"
#include "Misc/AsyncTaskNotification.h"
#include "WebAPILiquidJSCodeGenerator.h"
#include "WebAPILiquidJSProcess.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "WebAPILiquidJS"

void FWebAPILiquidJSModule::StartupModule()
{
	IWebAPIEditorModuleInterface& WebAPIEditorModule = IWebAPIEditorModuleInterface::Get();

	WebAPIEditorModule.AddCodeGenerator(TEXT("LiquidJS"), UWebAPILiquidJSCodeGenerator::StaticClass());

	WebApp = MakeShared<FWebAPILiquidJSProcess>();
	TSharedPtr<FWebAPILiquidJSProcess> WebAppLocal = WebApp;
	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([WebAppLocal]
	{
		WebAppLocal->TryStart();
	});
}

void FWebAPILiquidJSModule::ShutdownModule()
{
	WebApp->Shutdown();
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
	
	static FName WebAPIEditorModuleName = TEXT("WebAPIEditorModule");
	if(FModuleManager::Get().IsModuleLoaded(WebAPIEditorModuleName))
	{
		IWebAPIEditorModuleInterface& WebAPIEditorModule = IWebAPIEditorModuleInterface::Get();
		WebAPIEditorModule.RemoveCodeGenerator(TEXT("LiquidJS"));
	}
}

bool FWebAPILiquidJSModule::TryStartWebApp() const
{
	if(!WebApp.IsValid())
	{
		return false;
	}
	else if(WebApp->GetStatus() >= FWebAPILiquidJSProcess::EStatus::Launching)
	{
		return true;
	}

	return WebApp->TryStart();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebAPILiquidJSModule, WebAPILiquidJS)
