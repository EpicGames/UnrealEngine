// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebAPILiquidJSModule.h"

#include "Templates/SharedPointer.h"

class FWebAPILiquidJSModule final
    : public IWebAPILiquidJSModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	bool TryStartWebApp() const;

private:
	/** The actual process that runs the middleman server. */
	TSharedPtr<class FWebAPILiquidJSProcess> WebApp;
};
