// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPILiquidJSProcess.h"

#include "Misc/AsyncTaskNotification.h"
#include "SocketSubsystem.h"
#include "WebAPILiquidJSLog.h"
#include "WebAPILiquidJSSettings.h"
#include "Async/Async.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"

#define LOCTEXT_NAMESPACE "WebAPILiquidJS"

FWebAPILiquidJSProcess::FWebAPILiquidJSProcess()
	: Status(EStatus::Stopped)
	, bForceBuildWebApp(false)
{
}

bool FWebAPILiquidJSProcess::TryStart()
{
	Shutdown();

	Status = EStatus::Launching;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const FString WebApp = IPluginManager::Get().FindPlugin("WebAPI")->GetBaseDir() / TEXT("WebAPIGeneratorApp");
	Root = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*WebApp);

	if (!PlatformFile.DirectoryExists(*Root))
	{
		UE_LOGF(LogWebAPILiquidJS, Warning, "WebAPIGeneratorApp folder doesn't exist (%ls)", *Root);
		return false;
	}

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("WebAPILiquidJS_Launch", "Launching WebAPILiquidJS");
#if !NO_LOGGING
	NotificationConfig.LogCategory = &LogWebAPILiquidJS;
#endif
	NotificationConfig.bIsHeadless = false;
	TaskNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);

	Thread = FRunnableThread::Create(this, TEXT("FWebAPILiquidJSProcess"), 8 * 1024, TPri_BelowNormal);
	return true;
}

void FWebAPILiquidJSProcess::Shutdown()
{
	Status = EStatus::Stopped;

	SetExternalLoggerEnabled(false);

	if (Process.IsValid())
	{
		FPlatformProcess::TerminateProc(Process, true);
		Process.Reset();
	}

	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}

FWebAPILiquidJSProcess::EStatus FWebAPILiquidJSProcess::GetStatus() const
{
	return Status.load();
}

void FWebAPILiquidJSProcess::SetExternalLoggerEnabled(bool bEnableExternalLog) const
{
	if (bEnableExternalLog)
	{
		const UWebAPILiquidJSSettings* Settings = GetDefault<UWebAPILiquidJSSettings>();

		const TSharedRef<FInternetAddr> Address = ISocketSubsystem::Get()->CreateInternetAddr();

		bool bIsValidIp = false;
		Address->SetIp(TEXT("127.0.0.1"), bIsValidIp);
		Address->SetPort(Settings->Port + 2);
	}
}

uint32 FWebAPILiquidJSProcess::Run()
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

#if PLATFORM_WINDOWS
	const FString StartScript = Root / TEXT("Start.bat");
#elif PLATFORM_MAC
	const FString StartScript = Root / TEXT("Start.command");
#else
	const FString StartScript = Root / TEXT("Start.sh");
#endif

	const FText ErrorTitle = LOCTEXT("WebAPILiquidJS_ErrorTitle", "Failed to Launch the WebAPILiquidJS process");

	if (!PlatformFile.FileExists(*StartScript))
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("WebAPILiquidJS_FilesMissing", "Missing files in WebAPIGeneratorApp folder"),
			false
		);
		Status = EStatus::Error;
		return 0;
	}

	const UWebAPILiquidJSSettings* Settings = GetDefault<UWebAPILiquidJSSettings>();

	FString Args = FString::Printf(TEXT("--port %d --uews %d --uehttp %d --monitor "),
									Settings->Port,
									Settings->WebSocketServerPort,
									Settings->HttpServerPort);

	if (Settings->bForceWebAppBuildAtStartup || bForceBuildWebApp)
	{
		Args.Append(TEXT("--build "));
	}

	if (Settings->bWebAppLogRequestDuration)
	{
		Args.Append(TEXT("--log "));
	}

	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;

	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("WebAPILiquidJS_PipeFailed", "Failed to create Pipes for the WebAPILiquidJS process"),
			false
		);
		Status = EStatus::Error;
		return 0;
	}

	check(ReadPipe);
	check(WritePipe);

	Process = FPlatformProcess::CreateProc(
		*StartScript,	/* Path to start script */
		*Args,			/* Arguments with port numbers */
		false,			/* bLaunchDetached */
		true,			/* bLaunchHidden */
		true,			/* bLaunchReallyHidden */
		nullptr,		/* OutProcessID */
		0,				/* PriorityModifier */
		*Root,			/* OptionalWorkingDirectory */
		WritePipe		/* PipeWriteChild */
	);

	if (!Process.IsValid())
	{
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("WebAPILiquidJS_LaunchFailed", "Failed to start WebAPILiquidJS process"),
			false
		);

		Status = EStatus::Error;
		return 0;
	}

	bool bLoadDone = false;
	auto LogReadPipe = [&](void* InReadPipe)
	{
		const FString ProcessOutput = FPlatformProcess::ReadPipe(InReadPipe);
		if (ProcessOutput.Len() > 0)
		{
			TArray<FString> Lines;
			ProcessOutput.ParseIntoArray(Lines, TEXT("\n"), false);

			for (const FString& Line : Lines)
			{
				if (Line.Len() == 0)
				{
					continue;
				}

				if (!bLoadDone)
				{
					if (Line.StartsWith(TEXT("ERROR: ")))
					{
						bLoadDone = true;
						const FString ErrorMessage = Line.Mid(7);
						TaskNotification->SetComplete(ErrorTitle, FText::FromString(ErrorMessage), false);
						Status = EStatus::Error;
					}
					else if (Line.StartsWith(TEXT("DONE: ")))
					{
						bLoadDone = true;
						const FString CompleteMessage = Line.Mid(6);
						TaskNotification->SetComplete(
							LOCTEXT("WebAPILiquidJS_SuccessTitle", "WebAPILiquidJS is running"),
							FText::FromString(CompleteMessage),
							true);
						Status = EStatus::Running;

						if (Settings->bWebAppLogRequestDuration)
						{
							SetExternalLoggerEnabled(true);
						}
					}
					else
					{
						TaskNotification->SetProgressText(FText::FromString(Line));
					}
				}
				else
				{
					UE_LOGF(LogWebAPILiquidJS, Log, "%ls", *Line);
				}
			}
		}

		return ProcessOutput.Len();
	};

	UE_LOGF(LogWebAPILiquidJS, Log, "WebApp started, initial launch will take longer as it will be building the WebApp");

	while (FPlatformProcess::IsProcRunning(Process))
	{
		const int32 BytesRead = LogReadPipe(ReadPipe);
		if (!BytesRead)
		{
			FPlatformProcess::Sleep(0.2f);
		}
	}

	// One last ReadPipe log for anything that the while loop didn't catch.
	LogReadPipe(ReadPipe);

	if (!bLoadDone)
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("WebAPILiquidJS_LaunchFailed2", "WebApp exited"),
			false
		);
	}
	else
	{
		UE_LOGF(LogWebAPILiquidJS, Log, "WebApp exited");
	}

	EStatus PreStopStatus = Status;
	Status = EStatus::Stopped;

	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
	Process.Reset();

	// If haven't already, try again with --build flag
	if(PreStopStatus == EStatus::Error && !bForceBuildWebApp && !Settings->bForceWebAppBuildAtStartup)
	{
		bForceBuildWebApp = true;

		Async(EAsyncExecution::TaskGraphMainThread, [this]()
		{
			FAsyncTaskNotificationConfig NotificationConfig;
			NotificationConfig.TitleText = LOCTEXT("WebAPILiquidJS_Launch", "Launching WebAPILiquidJS");
	#if !NO_LOGGING
			NotificationConfig.LogCategory = &LogWebAPILiquidJS;
	#endif
			NotificationConfig.bIsHeadless = false;
			
			TaskNotification.Reset();
			TaskNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
		}).Wait();

		return Run();
	}

	return 0;
}

#undef LOCTEXT_NAMESPACE
