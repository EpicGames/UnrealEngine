// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Templates/UniquePtr.h"

class FAsyncTaskNotification;
class FRunnableThread;
struct FScriptContainerElement;

class FWebAPILiquidJSProcess : private FRunnable
{
public:
	enum class EStatus : uint8
	{
		Stopped = 0,
		Error = 1,
		
		Launching = 2,
		Running = 3,
	};

public:
	FWebAPILiquidJSProcess();
	
	virtual ~FWebAPILiquidJSProcess() override = default;

	/** Starts the webapp external process, first launch will take some time as it needs to compile the webapp */
	bool TryStart();

	/** Shutdown the webapp external process */
	void Shutdown();

	/** Get the webapp's status. */
	EStatus GetStatus() const;

	/** Enable / disable external logger */
	void SetExternalLoggerEnabled(bool bEnableExternalLog) const;

protected:
	virtual uint32 Run() override;

private:
	/** Plugin root folder */
	FString Root;

	/** WebApp Node.js process handle */
	FProcHandle Process;

	/** Thread for WebApp node.js process (read stdout and forward to log) */
	FRunnableThread* Thread = nullptr;

	/** Progress notification */
	TUniquePtr<FAsyncTaskNotification> TaskNotification;

	/** The current status of the web app. */
	std::atomic<EStatus> Status;

	/** Can by used override Settings for retry. */
	std::atomic<bool> bForceBuildWebApp;
};
