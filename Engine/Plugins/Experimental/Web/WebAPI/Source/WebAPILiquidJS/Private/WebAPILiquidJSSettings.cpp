// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPILiquidJSSettings.h"

FString UWebAPILiquidJSSettings::GetServiceUrl(const FString& InSubPath)
{
	// Remove prefix / to avoid double-up
	FString SubPath = InSubPath;
	SubPath.RemoveFromStart(TEXT("/"));
	
	if(Port != FormattedWithPort)
	{
		FormattedServiceUrl = FString::Format(*ServiceUrl, FStringFormatNamedArguments(
			{{	TEXT("Port"), Port }}
		));
		FormattedServiceUrl.RemoveFromEnd(TEXT("/"));
		FormattedWithPort = Port;

		check(!FormattedServiceUrl.Contains(TEXT("{")));
		check(!FormattedServiceUrl.Contains(TEXT("}")));
	}

	return FormattedServiceUrl + TEXT("/") + SubPath;
}

FString UWebAPILiquidJSSettings::GetServiceUrl(const FString& InSubPath) const
{
	if(Port != FormattedWithPort)
	{
		// Need to call non-const version before this is ever called!
		checkNoEntry();
	}

	FString SubPath = InSubPath;
	SubPath.RemoveFromStart(TEXT("/"));

	return FormattedServiceUrl + TEXT("/") + SubPath;
}
