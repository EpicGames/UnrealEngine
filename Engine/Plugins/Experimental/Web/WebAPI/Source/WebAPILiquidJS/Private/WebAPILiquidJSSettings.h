// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "WebAPILiquidJSSettings.generated.h"

/** */
UCLASS(BlueprintType, Config = Engine, DefaultConfig, meta = (DisplayName = "WebAPI LiquidJS"))
class WEBAPILIQUIDJS_API UWebAPILiquidJSSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/** The web app http port. */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Service")
	uint32 Port = 33000;

	/** Should force a build of the WebApp at startup. */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Service")
	bool bForceWebAppBuildAtStartup = false;

	/** Should WebApp log timing. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service")
	bool bWebAppLogRequestDuration = false;

	/** Whether web server is started automatically. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service")
	bool bAutoStartWebServer = true;

	/** Whether web socket server is started automatically. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service")
	bool bAutoStartWebSocketServer = true;

	/** The HTTP server's port. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service")
	uint32 HttpServerPort = 33010;

	/** The WebSocket server's port. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service")
	uint32 WebSocketServerPort = 33020;

	/** Returns a formatted Url. */
	FString GetServiceUrl(const FString& InSubPath = {});

	/** Returns a formatted Url. */
	FString GetServiceUrl(const FString& InSubPath = {}) const;

private:
	/** The web app url. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Service", meta = (AllowPrivateAccess))
	FString ServiceUrl = TEXT("127.0.0.1:{Port}");
	
	/** Cached formatted Url. */
	UPROPERTY(Transient)
	FString FormattedServiceUrl;

	/** What port the formatted Url was created with, used to re-cache on port change. */
	UPROPERTY(Transient)
	uint32 FormattedWithPort = 0;
};
