// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebAPILiquidJS : ModuleRules
{
	public WebAPILiquidJS(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"DeveloperSettings",
				"EditorStyle",
				"EditorWidgets",
				"Engine",
				"HTTP",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore",
				"Sockets",
				"UnrealEd",
				"WebAPI",
				"WebAPIEditor",
				"WebSocketNetworking",
			});
	}
}
