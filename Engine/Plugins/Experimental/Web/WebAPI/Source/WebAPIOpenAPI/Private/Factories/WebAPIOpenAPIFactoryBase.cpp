// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOpenAPIFactoryBase.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPIFactoryBase"

const FName UWebAPIOpenAPIFactoryBase::JsonFileType = FName(TEXT("json"));
const FName UWebAPIOpenAPIFactoryBase::YamlFileType = FName(TEXT("yaml"));

UWebAPIOpenAPIFactoryBase::UWebAPIOpenAPIFactoryBase()
{
	Formats.Add(TEXT("json;JavaScript Object Notation"));
}

bool UWebAPIOpenAPIFactoryBase::IsValidFileExtension(const FString& InFileExtension) const
{
	return InFileExtension.Equals(JsonFileType.ToString(), ESearchCase::IgnoreCase);
}

#undef LOCTEXT_NAMESPACE
