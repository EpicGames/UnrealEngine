// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Factories/WebAPIDefinitionFactory.h"

#include "WebAPIOpenAPIFactoryBase.generated.h"

/** Common base factory between Swagger and OpenAPI factories. */
UCLASS(Abstract)
class UWebAPIOpenAPIFactoryBase
	: public UWebAPIDefinitionFactory
{
	GENERATED_BODY()

public:
	UWebAPIOpenAPIFactoryBase();

	virtual TFuture<bool> ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents) override
	PURE_VIRTUAL(UWebAPIOpenAPIFactoryBase::ImportWebAPI, return MakeFulfilledPromise<bool>(false).GetFuture();)
	
	virtual bool IsValidFileExtension(const FString& InFileExtension) const override;

protected:
	static const FName JsonFileType;
	static const FName YamlFileType;
};
