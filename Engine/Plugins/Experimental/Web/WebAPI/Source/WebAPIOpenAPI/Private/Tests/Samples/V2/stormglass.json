{
  "swagger": "2.0",
  "schemes": [
    "https"
  ],
  "host": "api.stormglass.io",
  "basePath": "/",
  "info": {
    "contact": {
      "email": "support@stormglass.io"
    },
    "description": "Global marine weather data from multiple sources in one single API with hourly resolution. Get your API key by visiting the Storm Glass web site.",
    "termsOfService": "https://stormglass.io",
    "title": "Storm Glass Marine Weather",
    "version": "1.0.1",
    "x-apisguru-categories": [
      "location"
    ],
    "x-logo": {
      "backgroundColor": "#FFFFFF",
      "url": "https://api.apis.guru/v2/cache/logo/https_stormglass.io_wp-content_uploads_2019_05_Stormglass-Blue-circled-5.svg"
    },
    "x-origin": [
      {
        "format": "swagger",
        "url": "https://www.stormglass.io/specifications/api-specifications.yaml",
        "version": "2.0"
      }
    ],
    "x-providerName": "stormglass.io"
  },
  "produces": [
    "application/json"
  ],
  "securityDefinitions": {
    "authenticationToken": {
      "in": "header",
      "name": "Authentication-Token",
      "type": "apiKey"
    }
  },
  "tags": [
    {
      "description": "Get forecast for coordinates",
      "externalDocs": {
        "description": "Find out more",
        "url": "https://stormglass.io"
      },
      "name": "forecast"
    }
  ],
  "paths": {
    "/forecast": {
      "get": {
        "description": "Get forecast info for the given coordinates. For every hour and property, you will get a list of weather sources and their values.",
        "operationId": "getForecast",
        "parameters": [
          {
            "description": "The latitude for a location. Valid input is a number between -90 and 90.",
            "in": "query",
            "name": "lat",
            "required": true,
            "type": "number"
          },
          {
            "description": "The longitude for a location. Valid input is a number between -180 and 180.",
            "in": "query",
            "name": "lng",
            "required": true,
            "type": "number"
          }
        ],
        "responses": {
          "200": {
            "description": "Successful request",
            "schema": {
              "$ref": "#/definitions/Forecast",
              "type": "object"
            }
          },
          "403": {
            "description": "Unauthorized or API key is invalid"
          },
          "422": {
            "description": "Invalid or missing coordinates"
          }
        },
        "security": [
          {
            "authenticationToken": []
          }
        ],
        "summary": "Get hourly forecasts by coordinates",
        "tags": [
          "forecast"
        ]
      }
    }
  },
  "definitions": {
    "Forecast": {
      "properties": {
        "hours": {
          "description": "Every hour with forecast data is represented with an object in this array",
          "items": {
            "description": "Hourly forecast information",
            "properties": {
              "airTemperature": {
                "items": {
                  "description": "Air temperature given in celsius",
                  "properties": {
                    "source": {
                      "example": "smhi",
                      "type": "string"
                    },
                    "value": {
                      "example": "-2.6",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "swellDirection": {
                "items": {
                  "description": "Direction given in degrees where 0 goes from North to South",
                  "properties": {
                    "source": {
                      "example": "nooa",
                      "type": "string"
                    },
                    "value": {
                      "example": "164.35",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "swellHeight": {
                "items": {
                  "description": "Swell height given in meters",
                  "properties": {
                    "source": {
                      "example": "nooa",
                      "type": "string"
                    },
                    "value": {
                      "example": "2.94",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "swellPeriod": {
                "items": {
                  "description": "Swell period given in seconds",
                  "properties": {
                    "source": {
                      "example": "nooa",
                      "type": "string"
                    },
                    "value": {
                      "example": "15.88",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "time": {
                "description": "Timestamp in utc for the hour represented",
                "example": "2018-01-19T17:00:00+00:00",
                "type": "string"
              },
              "waterTemperature": {
                "items": {
                  "description": "Water temperature given in celsius",
                  "properties": {
                    "source": {
                      "example": "fcoo",
                      "type": "string"
                    },
                    "value": {
                      "example": "9",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "waveDirection": {
                "items": {
                  "description": "Direction given in degrees where 0 goes from North to South",
                  "properties": {
                    "source": {
                      "example": "fcoo",
                      "type": "string"
                    },
                    "value": {
                      "example": "121.35698599493828",
                      "type": "number"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "waveHeight": {
                "items": {
                  "description": "Wave height given in metres",
                  "properties": {
                    "source": {
                      "example": "fmi",
                      "type": "string"
                    },
                    "value": {
                      "example": "0.5969999991357327",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "wavePeriod": {
                "items": {
                  "properties": {
                    "source": {
                      "example": "fcoo",
                      "type": "string"
                    },
                    "value": {
                      "example": "3.63850457084894",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "windDirection": {
                "items": {
                  "description": "Wind direction given in degrees where 0 goes from North to South",
                  "properties": {
                    "source": {
                      "example": "smhi",
                      "type": "string"
                    },
                    "value": {
                      "example": "36.0",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              },
              "windSpeed": {
                "items": {
                  "description": "Wind speed given in meters per second",
                  "properties": {
                    "source": {
                      "example": "smhi",
                      "type": "string"
                    },
                    "value": {
                      "example": "36.0",
                      "type": "string"
                    }
                  },
                  "type": "object"
                },
                "type": "array"
              }
            },
            "type": "object"
          },
          "type": "array"
        },
        "meta": {
          "properties": {
            "dailyQuota": {
              "description": "The number of successful requests you can do on one day",
              "example": 100,
              "type": "integer"
            },
            "lat": {
              "description": "The latitude the response is showing forecast data for",
              "example": 58.7984,
              "type": "number"
            },
            "lng": {
              "description": "The longitude the response is showing forecast data for",
              "example": 17.80814,
              "type": "number"
            },
            "requestCount": {
              "description": "The number of successful requests made this day.",
              "example": 5,
              "type": "integer"
            }
          },
          "type": "object"
        }
      },
      "type": "object"
    }
  }
}