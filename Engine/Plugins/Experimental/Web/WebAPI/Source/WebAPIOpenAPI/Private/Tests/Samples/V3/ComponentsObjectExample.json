{
  "components": {
    "schemas": {
      "GeneralError": {
        "type": "object",
        "properties": {
          "code": {
            "type": "integer",
            "format": "int32"
          },
          "message": {
            "type": "string"
          }
        }
      },
      "Category": {
        "type": "object",
        "properties": {
          "id": {
            "type": "integer",
            "format": "int64"
          },
          "name": {
            "type": "string"
          }
        }
      },
      "Tag": {
        "type": "object",
        "properties": {
          "id": {
            "type": "integer",
            "format": "int64"
          },
          "name": {
            "type": "string"
          }
        }
      }
    },
    "parameters": {
      "skipParam": {
        "name": "skip",
        "in": "query",
        "description": "number of items to skip",
        "required": true,
        "schema": {
          "type": "integer",
          "format": "int32"
        }
      },
      "limitParam": {
        "name": "limit",
        "in": "query",
        "description": "max records to return",
        "required": true,
        "schema": {
          "type": "integer",
          "format": "int32"
        }
      }
    },
    "responses": {
      "NotFound": {
        "description": "Entity not found."
      },
      "IllegalInput": {
        "description": "Illegal input for operation."
      },
      "GeneralError": {
        "description": "General Error",
        "content": {
          "application/json": {
            "schema": {
              "$ref": "#/components/schemas/GeneralError"
            }
          }
        }
      }
    },
    "securitySchemes": {
      "api_key": {
        "type": "apiKey",
        "name": "api_key",
        "in": "header"
      },
      "petstore_auth": {
        "type": "oauth2",
        "flows": {
          "implicit": {
            "authorizationUrl": "http://example.org/api/oauth/dialog",
            "scopes": {
              "write:pets": "modify pets in your account",
              "read:pets": "read your pets"
            }
          }
        }
      }
    }
  }
}