{
  "name": "API Support",
  "url": "http://www.example.com/support",
  "email": "support@example.com"
}