{
  "description": "The number of allowed requests in the current period",
  "schema": {
    "type": "integer"
  }
}