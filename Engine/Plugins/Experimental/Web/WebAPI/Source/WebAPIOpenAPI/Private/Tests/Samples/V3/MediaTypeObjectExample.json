{
  "application/json": {
    "schema": {
      "$ref": "#/components/schemas/Pet"
    },
    "examples": {
      "cat" : {
        "summary": "An example of a cat",
        "value":
        {
          "name": "Fluffy",
          "petType": "Cat",
          "color": "White",
          "gender": "male",
          "breed": "Persian"
        }
      },
      "dog": {
        "summary": "An example of a dog with a cat's name",
        "value" :  {
          "name": "Puma",
          "petType": "Dog",
          "color": "Black",
          "gender": "Female",
          "breed": "Mixed"
        },
        "frog": {
          "$ref": "#/components/examples/frog-example"
        }
      }
    }
  }
}