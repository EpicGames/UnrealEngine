{
  "in": "query",
  "name": "freeForm",
  "schema": {
    "type": "object",
    "additionalProperties": {
      "type": "integer"
    }
  },
  "style": "form"
}