{
  "$ref": "Pet.json"
}