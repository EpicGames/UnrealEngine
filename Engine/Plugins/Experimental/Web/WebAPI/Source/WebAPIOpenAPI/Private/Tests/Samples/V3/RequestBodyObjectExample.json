{
  "description": "user to add to the system",
  "content": {
    "application/json": {
      "schema": {
        "$ref": "#/components/schemas/User"
      },
      "examples": {
        "user" : {
          "summary": "User Example",
          "externalValue": "http://foo.bar/examples/user-example.json"
        }
      }
    },
    "application/xml": {
      "schema": {
        "$ref": "#/components/schemas/User"
      },
      "examples": {
        "user" : {
          "summary": "User example in XML",
          "externalValue": "http://foo.bar/examples/user-example.xml"
        }
      }
    },
    "text/plain": {
      "examples": {
        "user" : {
          "summary": "User example in Plain text",
          "externalValue": "http://foo.bar/examples/user-example.txt"
        }
      }
    },
    "*/*": {
      "examples": {
        "user" : {
          "summary": "User example in other format",
          "externalValue": "http://foo.bar/examples/user-example.whatever"
        }
      }
    }
  }
}