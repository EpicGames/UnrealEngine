{
  "200": {
    "description": "a pet to be returned",
    "content": {
      "application/json": {
        "schema": {
          "$ref": "#/components/schemas/Pet"
        }
      }
    }
  },
  "default": {
    "description": "Unexpected error",
    "content": {
      "application/json": {
        "schema": {
          "$ref": "#/components/schemas/ErrorModel"
        }
      }
    }
  }
}