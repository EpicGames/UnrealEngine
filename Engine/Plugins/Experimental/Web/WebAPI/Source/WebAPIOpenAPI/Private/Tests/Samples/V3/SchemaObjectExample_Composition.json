{
  "components": {
    "schemas": {
      "ErrorModel": {
        "type": "object",
        "required": [
          "message",
          "code"
        ],
        "properties": {
          "message": {
            "type": "string"
          },
          "code": {
            "type": "integer",
            "minimum": 100,
            "maximum": 600
          }
        }
      },
      "ExtendedErrorModel": {
        "allOf": [
          {
            "$ref": "#/components/schemas/ErrorModel"
          },
          {
            "type": "object",
            "required": [
              "rootCause"
            ],
            "properties": {
              "rootCause": {
                "type": "string"
              }
            }
          }
        ]
      }
    }
  }
}