{
  "components": {
    "schemas": {
      "Pet": {
        "type": "object",
        "discriminator": {
          "propertyName": "petType"
        },
        "properties": {
          "name": {
            "type": "string"
          },
          "petType": {
            "type": "string"
          }
        },
        "required": [
          "name",
          "petType"
        ]
      },
      "Cat": {
        "description": "A representation of a cat. Note that `Cat` will be used as the discriminator value.",
        "allOf": [
          {
            "$ref": "#/components/schemas/Pet"
          },
          {
            "type": "object",
            "properties": {
              "huntingSkill": {
                "type": "string",
                "description": "The measured skill for hunting",
                "default": "lazy",
                "enum": [
                  "clueless",
                  "lazy",
                  "adventurous",
                  "aggressive"
                ]
              }
            },
            "required": [
              "huntingSkill"
            ]
          }
        ]
      },
      "Dog": {
        "description": "A representation of a dog. Note that `Dog` will be used as the discriminator value.",
        "allOf": [
          {
            "$ref": "#/components/schemas/Pet"
          },
          {
            "type": "object",
            "properties": {
              "packSize": {
                "type": "integer",
                "format": "int32",
                "description": "the size of the pack the dog is from",
                "default": 0,
                "minimum": 0
              }
            },
            "required": [
              "packSize"
            ]
          }
        ]
      }
    }
  }
}