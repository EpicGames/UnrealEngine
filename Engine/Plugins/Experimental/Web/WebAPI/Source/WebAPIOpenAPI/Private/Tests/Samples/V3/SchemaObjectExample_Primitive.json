{
  "type": "string",
  "format": "email"
}