{
  "type": "object",
  "additionalProperties": {
    "type": "string"
  }
}