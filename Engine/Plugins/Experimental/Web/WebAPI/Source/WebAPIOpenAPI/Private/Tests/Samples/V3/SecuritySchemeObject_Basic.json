{
  "type": "http",
  "scheme": "basic"
}