{
  "url": "https://development.gigantic-server.com/v1",
  "description": "Development server"
}