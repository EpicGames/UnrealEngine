{
  "openapi": "3.0.0",
  "info": {
    "title": "MapBox - Static",
    "description": "The Mapbox Static API returns static maps and raster tiles from styles in the Mapbox Style Specification. See the Static (Classic) API documentation.",
    "version": "1.0.0"
  },
  "servers": [
    {
      "url": "https://api.mapbox.com"
    }
  ],
  "paths": {
    "/styles/v1/{username}/{style_id}/tiles/{tilesize}/{z}/{x}/{y}{@2x}": {
      "get": {
        "tags": ["Static Tiles"],
        "summary": "Retrieve a static map from a style",
        "description": "# a 2x tile: this is a 512x512 tile appropriate\n# for high-density displays like Retina displays\n$ curl \"https://api.mapbox.com/v4/mapbox.streets/1/0/0@2x.png?access_token=your-access-token\"",
        "parameters": [
          {
            "name": "access_token",
            "in": "query",
            "schema": {
              "type": "string"
            },
            "example": "{{mapbox_full_access_token}}"
          }
        ],
        "responses": {
          "200": {
            "description": "Successful response",
            "content": {
              "application/json": {}
            }
          }
        }
      }
    },
    "/styles/v1/umeralikhan/cjkjys5uq0sf62smvfmgavc9k/tiles/1/1/0": {
      "get": {
        "tags": ["General"],
        "summary": "Retrieve raster tiles from styles",
        "description": "# a 2x tile: this is a 512x512 tile appropriate\n# for high-density displays like Retina displays\n$ curl \"https://api.mapbox.com/v4/mapbox.streets/1/0/0@2x.png?access_token=your-access-token\"",
        "parameters": [
          {
            "name": "access_token",
            "in": "query",
            "schema": {
              "type": "string"
            },
            "example": "{{mapbox_full_access_token}}"
          }
        ],
        "responses": {
          "200": {
            "description": "Successful response",
            "content": {
              "application/json": {}
            }
          }
        }
      }
    },
    "/styles/v1/umeralikhan/cjkjys5uq0sf62smvfmgavc9k/wmts": {
      "get": {
        "tags": ["General"],
        "summary": "Retrieve a map's WMTS document",
        "description": "# a 2x tile: this is a 512x512 tile appropriate\n# for high-density displays like Retina displays\n$ curl \"https://api.mapbox.com/v4/mapbox.streets/1/0/0@2x.png?access_token=your-access-token\"",
        "parameters": [
          {
            "name": "access_token",
            "in": "query",
            "schema": {
              "type": "string"
            },
            "example": "{{mapbox_full_access_token}}"
          }
        ],
        "responses": {
          "200": {
            "description": "Successful response",
            "content": {
              "application/json": {}
            }
          }
        }
      }
    }
  }
}
