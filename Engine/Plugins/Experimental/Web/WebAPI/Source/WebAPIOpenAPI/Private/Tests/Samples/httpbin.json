{
    "basePath": "/", 
    "definitions": {}, 
    "host": "httpbin.org", 
    "info": {
      "contact": {
        "email": "me@kennethreitz.org", 
        "responsibleDeveloper": "Kenneth Reitz", 
        "responsibleOrganization": "Kenneth Reitz", 
        "url": "https://kennethreitz.org"
      }, 
      "description": "A simple HTTP Request & Response Service.<br/> <br/> <b>Run locally: </b> <code>$ docker run -p 80:80 kennethreitz/httpbin</code>", 
      "title": "httpbin.org", 
      "version": "0.9.2"
    }, 
    "paths": {
      "/absolute-redirect/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "Absolutely 302 Redirects n times.", 
          "tags": [
            "Redirects"
          ]
        }
      }, 
      "/anything": {
        "delete": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "patch": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "post": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "put": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "trace": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }
      }, 
      "/anything/{anything}": {
        "delete": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "patch": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "post": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "put": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }, 
        "trace": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Anything passed in request"
            }
          }, 
          "summary": "Returns anything passed in request data.", 
          "tags": [
            "Anything"
          ]
        }
      }, 
      "/base64/{value}": {
        "get": {
          "parameters": [
            {
              "default": "SFRUUEJJTiBpcyBhd2Vzb21l", 
              "in": "path", 
              "name": "value", 
              "type": "string"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "200": {
              "description": "Decoded base64 content."
            }
          }, 
          "summary": "Decodes base64url-encoded string.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/basic-auth/{user}/{passwd}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "user", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "passwd", 
              "type": "string"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "401": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using HTTP Basic Auth.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/bearer": {
        "get": {
          "parameters": [
            {
              "in": "header", 
              "name": "Authorization", 
              "schema": {
                "type": "string"
              }
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "401": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using bearer authentication.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/brotli": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Brotli-encoded data."
            }
          }, 
          "summary": "Returns Brotli-encoded data.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/bytes/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/octet-stream"
          ], 
          "responses": {
            "200": {
              "description": "Bytes."
            }
          }, 
          "summary": "Returns n random bytes generated with given seed", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/cache": {
        "get": {
          "parameters": [
            {
              "in": "header", 
              "name": "If-Modified-Since"
            }, 
            {
              "in": "header", 
              "name": "If-None-Match"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Cached response"
            }, 
            "304": {
              "description": "Modified"
            }
          }, 
          "summary": "Returns a 304 if an If-Modified-Since header or If-None-Match is present. Returns the same as a GET otherwise.", 
          "tags": [
            "Response inspection"
          ]
        }
      }, 
      "/cache/{value}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "value", 
              "type": "integer"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Cache control set"
            }
          }, 
          "summary": "Sets a Cache-Control header for n seconds.", 
          "tags": [
            "Response inspection"
          ]
        }
      }, 
      "/cookies": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Set cookies."
            }
          }, 
          "summary": "Returns cookie data.", 
          "tags": [
            "Cookies"
          ]
        }
      }, 
      "/cookies/delete": {
        "get": {
          "parameters": [
            {
              "allowEmptyValue": true, 
              "explode": true, 
              "in": "query", 
              "name": "freeform", 
              "schema": {
                "additionalProperties": {
                  "type": "string"
                }, 
                "type": "object"
              }, 
              "style": "form"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "200": {
              "description": "Redirect to cookie list"
            }
          }, 
          "summary": "Deletes cookie(s) as provided by the query string and redirects to cookie list.", 
          "tags": [
            "Cookies"
          ]
        }
      }, 
      "/cookies/set": {
        "get": {
          "parameters": [
            {
              "allowEmptyValue": true, 
              "explode": true, 
              "in": "query", 
              "name": "freeform", 
              "schema": {
                "additionalProperties": {
                  "type": "string"
                }, 
                "type": "object"
              }, 
              "style": "form"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "200": {
              "description": "Redirect to cookie list"
            }
          }, 
          "summary": "Sets cookie(s) as provided by the query string and redirects to cookie list.", 
          "tags": [
            "Cookies"
          ]
        }
      }, 
      "/cookies/set/{name}/{value}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "name", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "value", 
              "type": "string"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "200": {
              "description": "Set cookies and redirects to cookie list."
            }
          }, 
          "summary": "Sets a cookie and redirects to cookie list.", 
          "tags": [
            "Cookies"
          ]
        }
      }, 
      "/deflate": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Defalte-encoded data."
            }
          }, 
          "summary": "Returns Deflate-encoded data.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/delay/{delay}": {
        "delete": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }, 
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }, 
        "patch": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }, 
        "post": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }, 
        "put": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }, 
        "trace": {
          "parameters": [
            {
              "in": "path", 
              "name": "delay", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A delayed response."
            }
          }, 
          "summary": "Returns a delayed response (max of 10 seconds).", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/delete": {
        "delete": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's DELETE parameters."
            }
          }, 
          "summary": "The request's DELETE parameters.", 
          "tags": [
            "HTTP Methods"
          ]
        }
      }, 
      "/deny": {
        "get": {
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "200": {
              "description": "Denied message"
            }
          }, 
          "summary": "Returns page denied by robots.txt rules.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/digest-auth/{qop}/{user}/{passwd}": {
        "get": {
          "parameters": [
            {
              "description": "auth or auth-int", 
              "in": "path", 
              "name": "qop", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "user", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "passwd", 
              "type": "string"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "401": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using Digest Auth.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/digest-auth/{qop}/{user}/{passwd}/{algorithm}": {
        "get": {
          "parameters": [
            {
              "description": "auth or auth-int", 
              "in": "path", 
              "name": "qop", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "user", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "passwd", 
              "type": "string"
            }, 
            {
              "default": "MD5", 
              "description": "MD5, SHA-256, SHA-512", 
              "in": "path", 
              "name": "algorithm", 
              "type": "string"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "401": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using Digest Auth + Algorithm.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/digest-auth/{qop}/{user}/{passwd}/{algorithm}/{stale_after}": {
        "get": {
          "description": "allow settings the stale_after argument.\n", 
          "parameters": [
            {
              "description": "auth or auth-int", 
              "in": "path", 
              "name": "qop", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "user", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "passwd", 
              "type": "string"
            }, 
            {
              "default": "MD5", 
              "description": "MD5, SHA-256, SHA-512", 
              "in": "path", 
              "name": "algorithm", 
              "type": "string"
            }, 
            {
              "default": "never", 
              "in": "path", 
              "name": "stale_after", 
              "type": "string"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "401": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using Digest Auth + Algorithm.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/drip": {
        "get": {
          "parameters": [
            {
              "default": 2, 
              "description": "The amount of time (in seconds) over which to drip each byte", 
              "in": "query", 
              "name": "duration", 
              "required": false, 
              "type": "number"
            }, 
            {
              "default": 10, 
              "description": "The number of bytes to respond with", 
              "in": "query", 
              "name": "numbytes", 
              "required": false, 
              "type": "integer"
            }, 
            {
              "default": 200, 
              "description": "The response code that will be returned", 
              "in": "query", 
              "name": "code", 
              "required": false, 
              "type": "integer"
            }, 
            {
              "default": 2, 
              "description": "The amount of time (in seconds) to delay before responding", 
              "in": "query", 
              "name": "delay", 
              "required": false, 
              "type": "number"
            }
          ], 
          "produces": [
            "application/octet-stream"
          ], 
          "responses": {
            "200": {
              "description": "A dripped response."
            }
          }, 
          "summary": "Drips data over a duration after an optional initial delay.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/encoding/utf8": {
        "get": {
          "produces": [
            "text/html"
          ], 
          "responses": {
            "200": {
              "description": "Encoded UTF-8 content."
            }
          }, 
          "summary": "Returns a UTF-8 encoded body.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/etag/{etag}": {
        "get": {
          "parameters": [
            {
              "in": "header", 
              "name": "If-None-Match"
            }, 
            {
              "in": "header", 
              "name": "If-Match"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Normal response"
            }, 
            "412": {
              "description": "match"
            }
          }, 
          "summary": "Assumes the resource has the given etag and responds to If-None-Match and If-Match headers appropriately.", 
          "tags": [
            "Response inspection"
          ]
        }
      }, 
      "/get": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's query parameters."
            }
          }, 
          "summary": "The request's query parameters.", 
          "tags": [
            "HTTP Methods"
          ]
        }
      }, 
      "/gzip": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "GZip-encoded data."
            }
          }, 
          "summary": "Returns GZip-encoded data.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/headers": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's headers."
            }
          }, 
          "summary": "Return the incoming request's HTTP headers.", 
          "tags": [
            "Request inspection"
          ]
        }
      }, 
      "/hidden-basic-auth/{user}/{passwd}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "user", 
              "type": "string"
            }, 
            {
              "in": "path", 
              "name": "passwd", 
              "type": "string"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Sucessful authentication."
            }, 
            "404": {
              "description": "Unsuccessful authentication."
            }
          }, 
          "summary": "Prompts the user for authorization using HTTP Basic Auth.", 
          "tags": [
            "Auth"
          ]
        }
      }, 
      "/html": {
        "get": {
          "produces": [
            "text/html"
          ], 
          "responses": {
            "200": {
              "description": "An HTML page."
            }
          }, 
          "summary": "Returns a simple HTML document.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/image": {
        "get": {
          "produces": [
            "image/webp", 
            "image/svg+xml", 
            "image/jpeg", 
            "image/png", 
            "image/*"
          ], 
          "responses": {
            "200": {
              "description": "An image."
            }
          }, 
          "summary": "Returns a simple image of the type suggest by the Accept header.", 
          "tags": [
            "Images"
          ]
        }
      }, 
      "/image/jpeg": {
        "get": {
          "produces": [
            "image/jpeg"
          ], 
          "responses": {
            "200": {
              "description": "A JPEG image."
            }
          }, 
          "summary": "Returns a simple JPEG image.", 
          "tags": [
            "Images"
          ]
        }
      }, 
      "/image/png": {
        "get": {
          "produces": [
            "image/png"
          ], 
          "responses": {
            "200": {
              "description": "A PNG image."
            }
          }, 
          "summary": "Returns a simple PNG image.", 
          "tags": [
            "Images"
          ]
        }
      }, 
      "/image/svg": {
        "get": {
          "produces": [
            "image/svg+xml"
          ], 
          "responses": {
            "200": {
              "description": "An SVG image."
            }
          }, 
          "summary": "Returns a simple SVG image.", 
          "tags": [
            "Images"
          ]
        }
      }, 
      "/image/webp": {
        "get": {
          "produces": [
            "image/webp"
          ], 
          "responses": {
            "200": {
              "description": "A WEBP image."
            }
          }, 
          "summary": "Returns a simple WEBP image.", 
          "tags": [
            "Images"
          ]
        }
      }, 
      "/ip": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The Requester's IP Address."
            }
          }, 
          "summary": "Returns the requester's IP Address.", 
          "tags": [
            "Request inspection"
          ]
        }
      }, 
      "/json": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "An JSON document."
            }
          }, 
          "summary": "Returns a simple JSON document.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/links/{n}/{offset}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }, 
            {
              "in": "path", 
              "name": "offset", 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "200": {
              "description": "HTML links."
            }
          }, 
          "summary": "Generate a page containing n links to other pages which do the same.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/patch": {
        "patch": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's PATCH parameters."
            }
          }, 
          "summary": "The request's PATCH parameters.", 
          "tags": [
            "HTTP Methods"
          ]
        }
      }, 
      "/post": {
        "post": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's POST parameters."
            }
          }, 
          "summary": "The request's POST parameters.", 
          "tags": [
            "HTTP Methods"
          ]
        }
      }, 
      "/put": {
        "put": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's PUT parameters."
            }
          }, 
          "summary": "The request's PUT parameters.", 
          "tags": [
            "HTTP Methods"
          ]
        }
      }, 
      "/range/{numbytes}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "numbytes", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/octet-stream"
          ], 
          "responses": {
            "200": {
              "description": "Bytes."
            }
          }, 
          "summary": "Streams n random bytes generated with given seed, at given chunk size per packet.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/redirect-to": {
        "delete": {
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }, 
        "get": {
          "parameters": [
            {
              "in": "query", 
              "name": "url", 
              "required": true, 
              "type": "string"
            }, 
            {
              "in": "query", 
              "name": "status_code", 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }, 
        "patch": {
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }, 
        "post": {
          "parameters": [
            {
              "in": "formData", 
              "name": "url", 
              "required": true, 
              "type": "string"
            }, 
            {
              "in": "formData", 
              "name": "status_code", 
              "required": false, 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }, 
        "put": {
          "parameters": [
            {
              "in": "formData", 
              "name": "url", 
              "required": true, 
              "type": "string"
            }, 
            {
              "in": "formData", 
              "name": "status_code", 
              "required": false, 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }, 
        "trace": {
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302/3XX Redirects to the given URL.", 
          "tags": [
            "Redirects"
          ]
        }
      }, 
      "/redirect/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "302 Redirects n times.", 
          "tags": [
            "Redirects"
          ]
        }
      }, 
      "/relative-redirect/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "text/html"
          ], 
          "responses": {
            "302": {
              "description": "A redirection."
            }
          }, 
          "summary": "Relatively 302 Redirects n times.", 
          "tags": [
            "Redirects"
          ]
        }
      }, 
      "/response-headers": {
        "get": {
          "parameters": [
            {
              "allowEmptyValue": true, 
              "explode": true, 
              "in": "query", 
              "name": "freeform", 
              "schema": {
                "additionalProperties": {
                  "type": "string"
                }, 
                "type": "object"
              }, 
              "style": "form"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Response headers"
            }
          }, 
          "summary": "Returns a set of response headers from the query string.", 
          "tags": [
            "Response inspection"
          ]
        }, 
        "post": {
          "parameters": [
            {
              "allowEmptyValue": true, 
              "explode": true, 
              "in": "query", 
              "name": "freeform", 
              "schema": {
                "additionalProperties": {
                  "type": "string"
                }, 
                "type": "object"
              }, 
              "style": "form"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Response headers"
            }
          }, 
          "summary": "Returns a set of response headers from the query string.", 
          "tags": [
            "Response inspection"
          ]
        }
      }, 
      "/robots.txt": {
        "get": {
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "200": {
              "description": "Robots file"
            }
          }, 
          "summary": "Returns some robots.txt rules.", 
          "tags": [
            "Response formats"
          ]
        }
      }, 
      "/status/{codes}": {
        "delete": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }, 
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }, 
        "patch": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }, 
        "post": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }, 
        "put": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }, 
        "trace": {
          "parameters": [
            {
              "in": "path", 
              "name": "codes"
            }
          ], 
          "produces": [
            "text/plain"
          ], 
          "responses": {
            "100": {
              "description": "Informational responses"
            }, 
            "200": {
              "description": "Success"
            }, 
            "300": {
              "description": "Redirection"
            }, 
            "400": {
              "description": "Client Errors"
            }, 
            "500": {
              "description": "Server Errors"
            }
          }, 
          "summary": "Return status code or random status code if more than one are given", 
          "tags": [
            "Status codes"
          ]
        }
      }, 
      "/stream-bytes/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/octet-stream"
          ], 
          "responses": {
            "200": {
              "description": "Bytes."
            }
          }, 
          "summary": "Streams n random bytes generated with given seed, at given chunk size per packet.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/stream/{n}": {
        "get": {
          "parameters": [
            {
              "in": "path", 
              "name": "n", 
              "type": "int"
            }
          ], 
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "Streamed JSON responses."
            }
          }, 
          "summary": "Stream n JSON responses", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/user-agent": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "The request's User-Agent header."
            }
          }, 
          "summary": "Return the incoming requests's User-Agent header.", 
          "tags": [
            "Request inspection"
          ]
        }
      }, 
      "/uuid": {
        "get": {
          "produces": [
            "application/json"
          ], 
          "responses": {
            "200": {
              "description": "A UUID4."
            }
          }, 
          "summary": "Return a UUID4.", 
          "tags": [
            "Dynamic data"
          ]
        }
      }, 
      "/xml": {
        "get": {
          "produces": [
            "application/xml"
          ], 
          "responses": {
            "200": {
              "description": "An XML document."
            }
          }, 
          "summary": "Returns a simple XML document.", 
          "tags": [
            "Response formats"
          ]
        }
      }
    }, 
    "protocol": "https", 
    "schemes": [
      "https"
    ], 
    "swagger": "2.0", 
    "tags": [
      {
        "description": "Testing different HTTP verbs", 
        "name": "HTTP Methods"
      }, 
      {
        "description": "Auth methods", 
        "name": "Auth"
      }, 
      {
        "description": "Generates responses with given status code", 
        "name": "Status codes"
      }, 
      {
        "description": "Inspect the request data", 
        "name": "Request inspection"
      }, 
      {
        "description": "Inspect the response data like caching and headers", 
        "name": "Response inspection"
      }, 
      {
        "description": "Returns responses in different data formats", 
        "name": "Response formats"
      }, 
      {
        "description": "Generates random and dynamic data", 
        "name": "Dynamic data"
      }, 
      {
        "description": "Creates, reads and deletes Cookies", 
        "name": "Cookies"
      }, 
      {
        "description": "Returns different image formats", 
        "name": "Images"
      }, 
      {
        "description": "Returns different redirect responses", 
        "name": "Redirects"
      }, 
      {
        "description": "Returns anything that is passed to request", 
        "name": "Anything"
      }
    ]
  }
  