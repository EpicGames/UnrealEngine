// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"
#include "V3/WebAPIOpenAPIConverter.h"
#include "WebAPIDefinition.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FWebAPIOpenAPI3Spec,
				"Plugins.WebAPI.OpenAPI3",
				EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

	TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject> InputDefinition;
	TStrongObjectPtr<UWebAPIDefinition> OutputDefinition;
	//TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> Converter;
	//TStrongObjectPtr<UWebAPIDefinitionFactory> Factory;

	template <class SchemaObjectType>
	void LoadFromJson(const TSharedPtr<SchemaObjectType>& InSchemaObject, const TSharedPtr<FJsonObject>& InJsonObject)
	{
		InSchemaObject->FromJson(InJsonObject.ToSharedRef());
	}

	template <>
	void LoadFromJson(const TSharedPtr<UE::WebAPI::OpenAPI::V3::FPathsObject>& InSchemaObject, const TSharedPtr<FJsonObject>& InJsonObject)
	{
		UE::WebAPI::OpenAPI::V3::FPathsObject& SchemaObject = InSchemaObject.ToSharedRef().Get();
		UE::Json::FromJson(InJsonObject.ToSharedRef(), SchemaObject);
	}

	template <class SchemaObjectType>
	void LoadFromJson(TArray<TSharedPtr<SchemaObjectType>>& InSchemaObject, const TSharedPtr<FJsonValueArray>& InJsonValue)
	{
		const TArray<TSharedPtr<FJsonValue>>* JsonArray = nullptr;
		if(InJsonValue->TryGetArray(JsonArray))
		{
			for(const TSharedPtr<FJsonValue>& JsonItem : *JsonArray)
			{
				TSharedPtr<SchemaObjectType> SchemaObject = MakeShared<SchemaObjectType>();
				SchemaObject->FromJson(JsonItem->AsObject().ToSharedRef());
				InSchemaObject.Add(SchemaObject);
			}
		}
	}

	template <class DestinationType, class SourceType>
	typename TEnableIf<!std::is_same_v<DestinationType, SourceType>, void>::Type
	TryAssign(TSharedPtr<DestinationType>& InDst, const TSharedPtr<SourceType>& InSrc) { }

	template <class DestinationType, class SourceType>
	typename TEnableIf<std::is_same_v<DestinationType, SourceType>, void>::Type
	TryAssign(TSharedPtr<DestinationType>& InDst, const TSharedPtr<SourceType>& InSrc)
	{
		InDst = InSrc;
	}

	template <class SchemaObjectType>
	TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> InitializeForFile(
		const FString& InFile,
		const TUniqueFunction<void(const TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>&, const TSharedPtr<SchemaObjectType>&)>& InAttachFunc = {})
	{
		ensureAlways(FPaths::FileExists(InFile));

		const TSharedPtr<FJsonObject> JsonObject = LoadJson(InFile);

		TSharedPtr<SchemaObjectType> Input = MakeShared<SchemaObjectType>();
		LoadFromJson(Input, JsonObject);

		if (InAttachFunc)
		{
			InputDefinition = MakeShared<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>();
			InAttachFunc(InputDefinition, Input);
		}
		else
		{
			TryAssign(InputDefinition, Input);
		}

		OutputDefinition = TStrongObjectPtr(NewObject<UWebAPIDefinition>());

		//Factory = TStrongObjectPtr(NewObject<UWebAPIOpenAPIFactory>());

		return MakeShared<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter>(
			InputDefinition,
			OutputDefinition->GetWebAPISchema(),
			OutputDefinition->GetMessageLog().ToSharedRef(),
			OutputDefinition->GetProviderSettings());
	}

	template <class SchemaObjectType>
	TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> InitializeArrayForFile(
		const FString& InFile,
		const TUniqueFunction<void(const TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>&, const TArray<TSharedPtr<SchemaObjectType>>&)>& InAttachFunc = {})
	{
		ensureAlways(FPaths::FileExists(InFile));

		TSharedPtr<FJsonValueArray> JsonValueArray = LoadJsonArray(InFile);
		
		TArray<TSharedPtr<SchemaObjectType>> InputArray;
		LoadFromJson(InputArray, JsonValueArray);
		
		if(InAttachFunc)
		{
			InputDefinition = MakeShared<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>();
			InAttachFunc(InputDefinition, InputArray);
		}
		
		OutputDefinition = TStrongObjectPtr(NewObject<UWebAPIDefinition>());

		//Factory = TStrongObjectPtr(NewObject<UWebAPIOpenAPIFactory>());

		return MakeShared<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter>(
			InputDefinition,
			OutputDefinition->GetWebAPISchema(),
			OutputDefinition->GetMessageLog().ToSharedRef(),
			OutputDefinition->GetProviderSettings());
	}

	FString GetSampleFile(const FString& InName) const
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
		check(Plugin.IsValid());

		FString FilePath = FPaths::Combine(Plugin->GetBaseDir(),
			TEXT("Source"), TEXT("WebAPIOpenAPI"),
			TEXT("Private"), TEXT("Tests"), TEXT("Samples"), TEXT("V3"), InName + TEXT(".json"));
		ensure(FPaths::FileExists(FilePath));
		return FilePath;
	}

	FString GetAPISample() const
	{
		return GetSampleFile(TEXT("petstore_V3"));
	}

	TSharedPtr<FJsonObject> LoadJson(const FString& InFile) const
	{
		FString FileContents;
		FFileHelper::LoadFileToString(FileContents, *InFile);

		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(FileContents), JsonObject))
		{
			return JsonObject;
		}
	
		TArray<TSharedPtr<FJsonValue>> JsonValues;
		if (FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(FileContents), JsonValues))
		{
			return JsonValues.IsEmpty() ? nullptr : JsonValues[0]->AsObject();
		}

		return nullptr;
	}

	TSharedPtr<FJsonValueArray> LoadJsonArray(const FString& InFile) const
	{
		FString FileContents;
		FFileHelper::LoadFileToString(FileContents, *InFile);
	
		TArray<TSharedPtr<FJsonValue>> JsonValueArray;
		FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(FileContents), JsonValueArray);
		return  MakeShared<FJsonValueArray>(JsonValueArray);
	}

	template <class InputModelType, class OutputModelType>
	TObjectPtr<OutputModelType> FindNamedModel(const TArray<TObjectPtr<InputModelType>>& InModels, const FString& InName) const
	{
		static_assert(TIsDerivedFrom<InputModelType, UWebAPIModelBase>::Value, "InputModelType is not derived from UWebAPIModelBase.");
		static_assert(TIsDerivedFrom<OutputModelType, InputModelType>::Value, "OutputModelType is not derived from InputModelType.");
	
		const TObjectPtr<InputModelType>* FoundModel = InModels.FindByPredicate([InName](const TObjectPtr<InputModelType> InModelBase)
		{
			// @note: order matters!
			if constexpr (std::is_base_of_v<UWebAPIParameter, InputModelType>)
			{
				if(const TObjectPtr<UWebAPIParameter> Parameter = Cast<UWebAPIParameter>(InModelBase))
				{
					return Parameter->Name.ToString(true).Equals(InName);
				}
			}
			if constexpr (std::is_base_of_v<UWebAPIModel, InputModelType>)
			{
				if(const TObjectPtr<UWebAPIModel> Model = Cast<UWebAPIModel>(InModelBase))
				{
					return Model->Name.ToString(true).Equals(InName);
				}
			}
			if constexpr (std::is_base_of_v<UWebAPIEnum, InputModelType>)
			{
				if(const TObjectPtr<UWebAPIEnum> Enum = Cast<UWebAPIEnum>(InModelBase))
				{
					return Enum->Name.ToString(true).Equals(InName);
				}
			}

			return false;
		});
		
		return FoundModel ? Cast<OutputModelType>(*FoundModel) : nullptr;
	}

	template <class OutputModelType>
	TObjectPtr<OutputModelType> FindNamedModel(const TArray<TObjectPtr<OutputModelType>>& InModels, const FString& InName) const
	{
		static_assert(TIsDerivedFrom<OutputModelType, UWebAPIModelBase>::Value, "OutputModelType is not derived from UWebAPIModelBase.");

		const TObjectPtr<OutputModelType>* FoundModel = InModels.FindByPredicate([InName](const TObjectPtr<OutputModelType> InModelBase)
		{
			// @note: order matters!
			if constexpr (std::is_base_of_v<UWebAPIParameter, OutputModelType>)
			{
				if(const TObjectPtr<UWebAPIParameter> Parameter = Cast<UWebAPIParameter>(InModelBase))
				{
					return Parameter->Name.ToString(true).Equals(InName);
				}
			}
			if constexpr (std::is_base_of_v<UWebAPIModel, OutputModelType>)
			{
				if(const TObjectPtr<UWebAPIModel> Model = Cast<UWebAPIModel>(InModelBase))
				{
					return Model->Name.ToString(true).Equals(InName);
				}
			}
			if constexpr (std::is_base_of_v<UWebAPIEnum, OutputModelType>)
			{
				if(const TObjectPtr<UWebAPIEnum> Enum = Cast<UWebAPIEnum>(InModelBase))
                {
                	return Enum->Name.ToString(true).Equals(InName);
                }
			}

			return false;
		});
		
		return FoundModel ? Cast<OutputModelType>(*FoundModel) : nullptr;
	}

	TObjectPtr<UWebAPIProperty> FindNamedProperty(const TObjectPtr<UWebAPIModel>& InModel, const FString& InName, const FString& InTypeName = {}) const
	{
		const TObjectPtr<UWebAPIProperty>* FoundProperty = InModel->Properties.FindByPredicate([InName, InTypeName](const TObjectPtr<UWebAPIProperty> InProperty)
		{
			if(InTypeName.IsEmpty())
			{
				return InProperty->Name.ToString(true).Equals(InName);
			}
			else
			{
				return InProperty->Name.ToString(true).Equals(InName)
					&& InProperty->Type.ToString(true).Equals(InTypeName);
			}
		});
		
		return FoundProperty ? *FoundProperty : nullptr;
	}

//~ Begin FAutomationTestBase extensions
	template<typename ValueType> bool TestNull(const FString& What, TObjectPtr<ValueType> Pointer)
	{
		return FAutomationTestBase::TestNull(*What, Pointer.Get());
	}

	template<typename ValueType> bool TestNotNull(const FString& What, TObjectPtr<ValueType> Pointer)
	{
		return FAutomationTestBase::TestNotNull<ValueType>(*What, Pointer.Get());
	}

	bool TestFoundProperty(const FString& What, const TObjectPtr<UWebAPIModel>& InModel, const FString& InName, const FString& InTypeName = {})
	{
		const TObjectPtr<UWebAPIProperty>* FoundProperty = InModel->Properties.FindByPredicate([InName, InTypeName](const TObjectPtr<UWebAPIProperty> InProperty)
		{
			if(InTypeName.IsEmpty())
			{
				return InProperty->Name.ToString(true).Equals(InName);
			}
			else
			{
				return InProperty->Name.ToString(true).Equals(InName)
					&& InProperty->Type.ToString(true).Equals(InTypeName);
			}
		});

		if (FoundProperty == nullptr)
		{
			if(InTypeName.IsEmpty())
			{
				AddError(FString::Printf(TEXT("Expected a property named '%s'."), *InName), 1);
			}
			else
			{
				AddError(FString::Printf(TEXT("Expected a property named '%s' of type '%s'."), *InName, *InTypeName), 1);				
			}
			return false;
		}
		
		return TestTrue(What, true);
	}
//~ End FAutomationTestBase Interface

END_DEFINE_SPEC(FWebAPIOpenAPI3Spec)

void FWebAPIOpenAPI3Spec::Define()
{
	BeforeEach([this]
	{
		//Factory.Reset();
		InputDefinition.Reset();
		OutputDefinition.Reset();
	});

	Describe("Petstore", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetAPISample();
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FOpenAPIObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ComponentsObject", [this]
	{
		It("Converts", [this]
		{
			const FString FilePath = GetSampleFile("ComponentsObjectExample");
			const TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> Converter = InitializeForFile<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>(FilePath);

			const bool bWasConverted = Converter->Convert();
			if(TestTrue("ComponentsObject converted to WebAPI", bWasConverted))
			{
				const TArray<TObjectPtr<UWebAPIModelBase>> AllConvertedModels = OutputDefinition->GetWebAPISchema()->Models;

				// Test Models
				{
					TArray<TObjectPtr<UWebAPIModel>> ConvertedModels;
					for(const TObjectPtr<UWebAPIModelBase>& Model : AllConvertedModels)
					{
						if(Model->IsA(UWebAPIModel::StaticClass()))
						{
							ConvertedModels.Add(Cast<UWebAPIModel>(Model));
						}
					}
					
					// Can be populated with other models, ie. nested models so check that it contains AT LEAST 3
					TestTrue("Converted 3 models", ConvertedModels.Num() >= 3);

					const TObjectPtr<UWebAPIModel> GeneralErrorModel = FindNamedModel<UWebAPIModel>(ConvertedModels, TEXT("GeneralError"));
					if(TestNotNull("GeneralError model found", GeneralErrorModel))
					{
						TestFoundProperty("Code property found", GeneralErrorModel, TEXT("Code"), TEXT("int32"));
						TestFoundProperty("Message property found", GeneralErrorModel, TEXT("Message"), TEXT("String"));
					}

					const TObjectPtr<UWebAPIModel> CategoryModel = FindNamedModel<UWebAPIModel>(ConvertedModels, TEXT("Category"));
					if(TestNotNull("Category model found", CategoryModel))
					{
						TestFoundProperty("Id property found", CategoryModel, TEXT("Id"), TEXT("int64"));
						TestFoundProperty("Name property found", CategoryModel, TEXT("Name"), TEXT("String"));
					}

					const TObjectPtr<UWebAPIModel> TagModel = FindNamedModel<UWebAPIModel>(ConvertedModels, TEXT("Tag"));
					if(TestNotNull("Tag model found", TagModel))
					{
						TestFoundProperty("Id property found", TagModel, TEXT("Id"), TEXT("int64"));
						TestFoundProperty("Name property found", TagModel, TEXT("Name"), TEXT("String"));
					}
				}
			}
		});
	});
	
	Describe("PathsObject", [this]
	{
		It("Converts", [this]
		{
			const FString FilePath = GetSampleFile("PathsObjectExample");
			const TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> Converter = InitializeForFile<UE::WebAPI::OpenAPI::V3::FPathsObject>(
				FilePath,
				[](const TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject> InRootObject, const TSharedPtr<UE::WebAPI::OpenAPI::V3::FPathsObject>& InPathsObject)
				{
					InRootObject->Paths = *InPathsObject;
				});

			const bool bWasConverted = Converter->Convert();
			if(TestTrue("PathsObject converted to WebAPI", bWasConverted))
			{
				const TMap<FString, TObjectPtr<UWebAPIService>> ConvertedServices = OutputDefinition->GetWebAPISchema()->Services;
				TestTrue("Converted 1 service", ConvertedServices.Num() >= 1);

				TArray<TObjectPtr<UWebAPIService>> ServiceArray;
				ConvertedServices.GenerateValueArray(ServiceArray);
				const TObjectPtr<UWebAPIService> FirstService = ServiceArray[0];

				TestTrue("Service has 1 operation", FirstService->Operations.Num() >= 1);
				
				const TObjectPtr<UWebAPIOperation> GetPetsOperation = FirstService->Operations[0];
				if(TestNotNull("Get Pets operation found", GetPetsOperation))
				{
					TestTrue("Operation has 1 response", GetPetsOperation->Responses.Num() >= 1);

					const TObjectPtr<UWebAPIOperationResponse> FirstResponse = GetPetsOperation->Responses[0];
					if(TestEqual("Response is for code 200", FirstResponse->Code, 200))
					{
						TestTrue("Response has 1 property", FirstResponse->Properties.Num() >= 1);

						const TObjectPtr<UWebAPIProperty> FirstProperty = FirstResponse->Properties[0];
						if(TestEqual("Response contains a property named Pets", FirstProperty->Name.ToString(true), TEXT("Pets")))
						{
							TestTrue("Pets property is an array", FirstProperty->bIsArray);
							TestEqual("Pets property is stored in the message body", FirstResponse->Storage, EWebAPIResponseStorage::Body);
						}
					}
				}
			}
		});
	});

	Describe("ServersObject", [this]
	{
		It("Converts with Variables", [this]
		{
			const FString FilePath = GetSampleFile("ServerObjectExample_Variables");
			const TSharedPtr<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter> Converter = InitializeArrayForFile<UE::WebAPI::OpenAPI::V3::FServerObject>(
				FilePath,
				[](const TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject> InRootObject, const TArray<TSharedPtr<UE::WebAPI::OpenAPI::V3::FServerObject>>& InServersObject)
				{
					InRootObject->Servers = InServersObject;
				});

			const bool bWasConverted = Converter->Convert();
			if(TestTrue("ServersObject converted to WebAPI", bWasConverted))
			{
				const FString ExpectedUrl = TEXT("{username}.gigantic-server.com:{port}");
				TestEqual(FString::Printf(TEXT("Host is \"%s\""), *ExpectedUrl), OutputDefinition->GetWebAPISchema()->Host, ExpectedUrl);
			}
		});
	});

	Describe("InfoObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("InfoObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FInfoObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("OpenAPIObject", [this]
	{
		It("Parse", [this]
		{

		});
	});

	Describe("ContactObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ContactObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FContactObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("LicenseObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("LicenseObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FLicenseObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});



	Describe("ServerVariableObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ServerObjectExample_Variables");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FServerObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("PathItemObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("PathItemObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FPathItemObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ResponseObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ResponseObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FResponseObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("OperationObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("OperationObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FOperationObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ExternalDocumentationObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ExternalDocumentationObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FExternalDocumentationObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ParameterObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ParameterObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FParameterObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("RequestBodyObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("RequestBodyObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FRequestBodyObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("MediaTypeObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("MediaTypeObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FMediaTypeObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("EncodingObject", [this]
	{
		It("Parse", [this]
		{
			// const FString FilePath = GetSampleFile("EncodingObjectExample");
			// const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);
			//
			// UE::WebAPI::OpenAPI::V3::FEncodingObject Object;
			// Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("CallbackObject", [this]
	{
		It("Parse", [this]
		{
			// const FString FilePath = GetSampleFile("CallbackObjectExample");
			// const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);
			//
			// UE::WebAPI::OpenAPI::V3::FCallbackObject Object;
			// Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ExampleObject", [this]
	{
		It("Parse", [this]
		{
			// const FString FilePath = GetSampleFile("ExampleObjectExample");
			// const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);
			//
			// UE::WebAPI::OpenAPI::V3::FExampleObject Object;
			// Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("LinkObject", [this]
	{
		It("Parse", [this]
		{
			// const FString FilePath = GetSampleFile("LinkObjectExample");
			// const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);
			//
			// UE::WebAPI::OpenAPI::V3::FLinkObject Object;
			// Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("HeaderObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("HeaderObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FHeaderObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("TagObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("TagObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FTagObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("ReferenceObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("ReferenceObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FReferenceObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("SchemaObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("SchemaObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FSchemaObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("DiscriminatorObject", [this]
	{
		It("Parse", [this]
		{
			// const FString FilePath = GetSampleFile("DiscriminatorObjectExample");
			// const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);
			//
			// UE::WebAPI::OpenAPI::V3::FDiscriminatorObject Object;
			// Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("XMLObject", [this]
	{
		It("Parse", [this]
		{
			
		});
	});

	Describe("SecuritySchemeObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("SecuritySchemeObject_ApiKey");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FSecuritySchemeObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("OAuthFlowsObject", [this]
	{
		It("Parse", [this]
		{

		});
	});

	Describe("OAuthFlowObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("OAuthFlowObjectExample");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FOAuthFlowObject Object;
			Object.FromJson(JsonObject.ToSharedRef());
		});
	});

	Describe("SecurityRequirementObject", [this]
	{
		It("Parse", [this]
		{
			const FString FilePath = GetSampleFile("SecurityRequirementObjectExample_OAuth2");
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V3::FSecurityRequirementObject Object;
			UE::Json::FromJson(JsonObject.ToSharedRef(), Object);
			//Object.FromJson(JsonObject.ToSharedRef());
		});
	});
}

#endif
#endif
