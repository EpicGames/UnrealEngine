// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Interfaces/IPluginManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"
#include "V2/WebAPISwaggerConverter.h"
#include "V2/WebAPISwaggerSchema.h"
#include "WebAPIDefinition.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FWebAPISwaggerSpec,
	TEXT("Plugins.WebAPI.Swagger"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

	TSharedPtr<UE::WebAPI::OpenAPI::V2::FSwagger> InputDefinition;
	TStrongObjectPtr<UWebAPIDefinition> OutputDefinition;
	TSharedPtr<UE::WebAPI::Swagger::FWebAPISwaggerSchemaConverter> Converter;

	FString GetSampleFile(const FString& InName) const
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
		check(Plugin.IsValid());

		FString FilePath = FPaths::Combine(Plugin->GetBaseDir(),
			TEXT("Source"), TEXT("WebAPIOpenAPI"),
			TEXT("Private"), TEXT("Tests"), TEXT("Samples"), TEXT("V2"), InName + TEXT(".json"));
		ensure(FPaths::FileExists(FilePath));
		return FilePath;
	}

	FString GetAPISample() const
	{
		return GetSampleFile(TEXT("petstore_v2"));
	}

	TSharedPtr<FJsonObject> LoadJson(const FString& InFile) const
	{
		FString FileContents;
		FFileHelper::LoadFileToString(FileContents, *InFile);
	
		TSharedPtr<FJsonObject> JsonObject;
		FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(FileContents), JsonObject);
		return JsonObject;
	}

END_DEFINE_SPEC(FWebAPISwaggerSpec)

void FWebAPISwaggerSpec::Define()
{
	using namespace UE::WebAPI::OpenAPI;

	BeforeEach([this]
	{
		const FString FilePath = GetAPISample();
		const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

		InputDefinition = MakeShared<UE::WebAPI::OpenAPI::V2::FSwagger>();
		InputDefinition->FromJson(JsonObject.ToSharedRef());
					
		OutputDefinition = TStrongObjectPtr(NewObject<UWebAPIDefinition>());

		Converter = MakeShared<UE::WebAPI::Swagger::FWebAPISwaggerSchemaConverter>(
			InputDefinition,
			OutputDefinition->GetWebAPISchema(),
			OutputDefinition->GetMessageLog().ToSharedRef(),
			OutputDefinition->GetProviderSettings());
	});
	
	Describe("PetStore", [this]
	{
		It("Parses", [this]
		{
			const FString FilePath = GetAPISample();
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			UE::WebAPI::OpenAPI::V2::FSwagger Object;
			Object.FromJson(JsonObject.ToSharedRef());

			TestEqual("Swagger Version", Object.Swagger, "2.0");
			TestEqual("Info Version", Object.Info->Version, "1.0.5");
			if(TestTrue("Host Set", Object.Host.IsSet()))
			{
				TestEqual("Host", Object.Host.GetValue(), "petstore.swagger.io");
			}

			if(TestTrue("BasePath Set", Object.BasePath.IsSet()))
			{
				TestEqual("BasePath", Object.BasePath.GetValue(), "/v2");	
			}

			if(TestTrue("Tags Set", Object.Tags.IsSet()))
			{
				TestEqual("Tag Num", Object.Tags->Num(), 3);	
			}

			if(TestTrue("Schemes Set", Object.Schemes.IsSet()))
			{
				TestEqual("Scheme Num", Object.Schemes->Num(), 2);
			}

			TestEqual("Paths Num", Object.Paths.Num(), 14);
			
			if(TestTrue("Security Definitions Set", Object.SecurityDefinitions.IsSet()))
			{
				TestEqual("Security Definitions Num", Object.SecurityDefinitions->Num(), 2);
			}
			
			if(TestTrue("Definitions Set", Object.Definitions.IsSet()))
			{
				TestEqual("Definitions Num", Object.Definitions->Num(), 6);
			}

			if(TestTrue("External Docs Set", Object.ExternalDocs.IsSet()))
			{
				TestEqual("External Docs URL", Object.ExternalDocs.GetValue()->Url, TEXT("http://swagger.io"));
			}			
		});
	});

	Describe("Type Conversion", [this]
	{
		BeforeEach([this]
		{
			const FString FilePath = GetAPISample();
			const TSharedPtr<FJsonObject> JsonObject = LoadJson(FilePath);

			InputDefinition = MakeShared<UE::WebAPI::OpenAPI::V2::FSwagger>();
			InputDefinition->FromJson(JsonObject.ToSharedRef());
					
			OutputDefinition = TStrongObjectPtr(NewObject<UWebAPIDefinition>());

			Converter = MakeShared<UE::WebAPI::Swagger::FWebAPISwaggerSchemaConverter>(
				InputDefinition,
				OutputDefinition->GetWebAPISchema(),
				OutputDefinition->GetMessageLog().ToSharedRef(),
				OutputDefinition->GetProviderSettings());
		});

		It("Default value should be set in metadata", [this]
		{

		});
	});
}

#endif
#endif
