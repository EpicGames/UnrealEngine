// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPISwaggerConverter.h"

#include "Dom/WebAPIType.h"
#include "IWebAPIEditorModule.h"
#include "V2/WebAPISwaggerConverter.inl"
#include "V2/WebAPISwaggerProvider.h"
#include "WebAPIOpenAPILog.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPITypeRegistry.h"
#include "V2/WebAPISwaggerSchema.h"

#define LOCTEXT_NAMESPACE "WebAPISwaggerConverter"

#define SET_OPTIONAL(SrcProperty, DstProperty)			\
if(SrcProperty.IsSet())									\
{														\
	DstProperty = SrcProperty.GetValue();				\
}

#define SET_OPTIONAL_FLAGGED(SrcProperty, DstProperty, DstFlag)	\
if(SrcProperty.IsSet())											\
{																\
	DstFlag = true;												\
	DstProperty = SrcProperty.GetValue();						\
}

namespace UE::WebAPI::Swagger
{
	FWebAPISwaggerSchemaConverter::FWebAPISwaggerSchemaConverter(
		const TSharedPtr<const UE::WebAPI::OpenAPI::V2::FSwagger>& InSwagger, UWebAPISchema* InWebAPISchema,
		const TSharedRef<FWebAPIMessageLog>& InMessageLog, const FWebAPIProviderSettings& InProviderSettings): InputSchema(InSwagger)
		, OutputSchema(InWebAPISchema)
		, MessageLog(InMessageLog)
		, ProviderSettings(InProviderSettings)
	{
	}

	bool FWebAPISwaggerSchemaConverter::Convert()
	{
		OutputSchema->APIName = InputSchema->Info->Title;
		OutputSchema->Version = InputSchema->Info->Version;
		OutputSchema->Host = InputSchema->Host.Get(TEXT(""));
		OutputSchema->BaseUrl = InputSchema->BasePath.Get(TEXT(""));
		OutputSchema->URISchemes = InputSchema->Schemes.Get({ TEXT("https") }); // Get schemes or default to https

		// If Url isn't complete, there may not be a valid host
		if(OutputSchema->Host.IsEmpty())
		{
			MessageLog->LogWarning(LOCTEXT("NoHostProvided", "The specification did not contain a host Url, this should be specified manually in the generated project settings."), FWebAPISwaggerProvider::LogName);
		}
		
		// Top level decl of tags optional, so find from paths
		bool bSuccessfullyConverted = InputSchema->Tags.IsSet()
									? ConvertTags(InputSchema->Tags.GetValue(), OutputSchema.Get())
									: true;
		bSuccessfullyConverted &= ConvertModels(InputSchema->Definitions.Get({}), OutputSchema.Get());
		bSuccessfullyConverted &= ConvertParameters(InputSchema->Parameters.Get({}), OutputSchema.Get());
		bSuccessfullyConverted &= ConvertPaths(InputSchema->Paths, OutputSchema.Get());

		return bSuccessfullyConverted;
	}

	FString FWebAPISwaggerSchemaConverter::NameTransformer(const FWebAPINameVariant& InString) const
	{
		return ProviderSettings.ToPascalCase(InString);
	}

	TObjectPtr<UWebAPITypeInfo> FWebAPISwaggerSchemaConverter::ResolveType(
		FString InJsonType,
		FString InTypeHint,
		FString InDefinitionName,
		const TSharedPtr<OpenAPI::V2::FSchema>& InSchema) const
	{
		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();

		static TMap<EJson, FString> EJson_ValueToTypeName = {
			{EJson::None, TEXT("None")},
			{EJson::Null, TEXT("nullptr")},
			{EJson::String, TEXT("FString")},
			{EJson::Boolean, TEXT("bool")},
			{EJson::Number, TEXT("int32")},
			{EJson::Array, TEXT("TArray")},
			{EJson::Object, TEXT("UObject")},
		};

		static TMap<EJson, TObjectPtr<UWebAPITypeInfo>> EJson_ValueToTypeInfo = {
			{EJson::None, StaticTypeRegistry->Void},
			{EJson::Null, StaticTypeRegistry->Nullptr},
			{EJson::String, StaticTypeRegistry->String},
			{EJson::Boolean, StaticTypeRegistry->Boolean},
			{EJson::Number, StaticTypeRegistry->Int32},
			{EJson::Array, StaticTypeRegistry->String},
			{EJson::Object, StaticTypeRegistry->Object},
		};

		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> JsonTypeValueToTypeInfo = {
			{TEXT("None"), StaticTypeRegistry->Void},
			{TEXT("Null"), StaticTypeRegistry->Nullptr},
			{TEXT("String"), StaticTypeRegistry->String},
			{TEXT("Boolean"), StaticTypeRegistry->Boolean},
			{TEXT("Number"), StaticTypeRegistry->Int32},
			{TEXT("Array"), StaticTypeRegistry->String},
			{TEXT("Object"), StaticTypeRegistry->Object},
		};

		// OpenAPI type to UE type (Prefix, Name)
		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> TypeMap =
		{
			{TEXT("File"), StaticTypeRegistry->Nullptr},
			{TEXT("file"), StaticTypeRegistry->FilePath},
			{TEXT("any"), StaticTypeRegistry->Object},
			{TEXT("object"), StaticTypeRegistry->Object},

			{TEXT("array"), StaticTypeRegistry->String},
			{TEXT("map"), StaticTypeRegistry->String},
			{TEXT("boolean"), StaticTypeRegistry->Boolean},
			{TEXT("byte"), StaticTypeRegistry->Byte},
			{TEXT("integer"), StaticTypeRegistry->Int32},
			{TEXT("int"), StaticTypeRegistry->Int32},
			{TEXT("int32"), StaticTypeRegistry->Int32},
			{TEXT("short"), StaticTypeRegistry->Int16},
			{TEXT("int16"), StaticTypeRegistry->Int16},
			{TEXT("long"), StaticTypeRegistry->Int64},
			{TEXT("int64"), StaticTypeRegistry->Int64},
			{TEXT("float"), StaticTypeRegistry->Float},
			{TEXT("double"), StaticTypeRegistry->Double},

			{TEXT("number"), StaticTypeRegistry->Int32},
			{TEXT("char"), StaticTypeRegistry->Char},
			{TEXT("date"), StaticTypeRegistry->DateTime},
			{TEXT("date-time"), StaticTypeRegistry->DateTime},
			{TEXT("password"), StaticTypeRegistry->String},
			{TEXT("string"), StaticTypeRegistry->String},
			{TEXT("void"), StaticTypeRegistry->Void},
			{TEXT("null"), StaticTypeRegistry->Nullptr}
		};

		TObjectPtr<UWebAPITypeInfo> Result = nullptr;

		// If a definition name is supplied, try to find it first
		if ((InJsonType == TEXT("Object") || InJsonType == TEXT("Array")) && !InDefinitionName.IsEmpty())
		{
			if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, InDefinitionName))
			{
				Result = *FoundTypeInfo;
			}
		}

		// If not found above
		if (!Result)
		{
			// Try specific types
			if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfoAlt = TypeMap.Find(InTypeHint.IsEmpty() ? InJsonType : InTypeHint))
			{
				Result = *FoundTypeInfoAlt;
			}
			// Fallback to basic types
			else if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = JsonTypeValueToTypeInfo.Find(InJsonType))
			{
				Result = *FoundTypeInfo;
			}

			// If it's not a built-in type
			if (Result != nullptr && !Result->bIsBuiltinType)
			{
				// Duplicate it with the provided definition name 
				if (!InDefinitionName.IsEmpty())
				{
					// Allow prefix to be set later depending on this?
					Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(EWebAPISchemaType::Model,
						InDefinitionName,
						InDefinitionName,
						Result);
					Result->Prefix = TEXT("F");
					ensure(!Result->Name.IsEmpty());
				}
				else
				{
					checkNoEntry();
					Result = Result->Duplicate(OutputSchema->TypeRegistry);
				}
			}
		}

		return Result;
	}

	TObjectPtr<UWebAPITypeInfo> FWebAPISwaggerSchemaConverter::GetTypeForContentType(const FString& InContentType)
	{
		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
		
		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> ContentTypeToTypeInfo = {
			{TEXT("application/json"), StaticTypeRegistry->JsonObject},
			{TEXT("application/xml"), StaticTypeRegistry->String},
			{TEXT("text/plain"), StaticTypeRegistry->String}
		};
		
		if(const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = ContentTypeToTypeInfo.Find(InContentType.ToLower()))
		{
			return *FoundTypeInfo;
		}

		// For all other cases, use string
		return StaticTypeRegistry->String;
	}

	template <>
	FString FWebAPISwaggerSchemaConverter::GetDefaultJsonTypeForStorage<EWebAPIParameterStorage>(const EWebAPIParameterStorage& InStorage)
	{
		static TMap<EWebAPIParameterStorage, FString> StorageToJsonType = {
			{ EWebAPIParameterStorage::Body, TEXT("object") },
			{ EWebAPIParameterStorage::Cookie, TEXT("string") },
			{ EWebAPIParameterStorage::Header, TEXT("string") },
			{ EWebAPIParameterStorage::Path, TEXT("string") },
			{ EWebAPIParameterStorage::Query, TEXT("string") },
		};

		return StorageToJsonType[InStorage];
	}

	template <>
	FString FWebAPISwaggerSchemaConverter::GetDefaultJsonTypeForStorage<EWebAPIResponseStorage>(const EWebAPIResponseStorage& InStorage)
	{
		static TMap<EWebAPIResponseStorage, FString> StorageToJsonType = {
			{ EWebAPIResponseStorage::Body, TEXT("object") },
			{ EWebAPIResponseStorage::Header, TEXT("string") },
		};

		return StorageToJsonType[InStorage];
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPITypeInfo> FWebAPISwaggerSchemaConverter::ResolveType(const TSharedPtr<SchemaType>& InSchema, const FString& InDefinitionName, FString InJsonType)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();

		FString DefinitionName = InDefinitionName;

		if(InSchema->Items.IsSet())
		{
			const TSharedPtr<UE::WebAPI::OpenAPI::V2::FSchema> ItemSchema = ResolveReference(InSchema->Items.GetValue(), DefinitionName);
			if(!InSchema->Items->GetPath().IsEmpty())
			{
				DefinitionName = InSchema->Items->GetLastPathSegment();						
			}
			return ResolveType(ItemSchema, DefinitionName);
		}

		TObjectPtr<UWebAPITypeInfo> Result = ResolveType(
			InSchema->Type.Get(InJsonType.IsEmpty() ? TEXT("object") : InJsonType),
			InSchema->Format.Get(TEXT("")),
			DefinitionName);
		if (InSchema->Enum.IsSet() && !InSchema->Enum.GetValue().IsEmpty())
		{
			if(const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, NameTransformer(DefinitionName)))
			{
				Result = *FoundGeneratedType;						
			}
			else
			{
				Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
					EWebAPISchemaType::Model,
					NameTransformer(DefinitionName),
					DefinitionName,
					StaticTypeRegistry->Enum);
			}
		}
		else if (!Result->IsEnum() && (Result == StaticTypeRegistry->Object || Result->ToString(true).IsEmpty()))
		{
			if (!DefinitionName.IsEmpty())
			{
				if (const TObjectPtr<UWebAPITypeInfo>* FoundBuiltinType = StaticTypeRegistry->FindBuiltinType(DefinitionName))
				{
					Result = *FoundBuiltinType;
				}
				else if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedModelType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, DefinitionName))
				{
					Result = *FoundGeneratedModelType;
				}
				else if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedParameterType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Parameter, DefinitionName))
				{
					Result = *FoundGeneratedParameterType;
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("DefinitionName"), FText::FromString(DefinitionName));
					MessageLog->LogInfo(FText::Format(LOCTEXT("CannotResolveType", "ResolveType (object) failed to find a matching type for definition \"{DefinitionName}\", creating a new one."), Args), FWebAPISwaggerProvider::LogName);

					Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
						EWebAPISchemaType::Model,
						NameTransformer(DefinitionName),
						{},
						TEXT("F"));
					Result->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
				}
			}
		}

		check(Result);

		return Result;
	}

	template <>
	TSharedPtr<OpenAPI::V2::FSchema> FWebAPISwaggerSchemaConverter::ResolveReference<OpenAPI::V2::FSchema>(const FString& InDefinitionName)
	{
		if (const TSharedPtr<OpenAPI::V2::FSchema>* FoundDefinition = InputSchema->Definitions->Find(InDefinitionName))
		{
			return *FoundDefinition;
		}

		return nullptr;
	}

	template <>
	TSharedPtr<OpenAPI::V2::FParameter> FWebAPISwaggerSchemaConverter::ResolveReference<OpenAPI::V2::FParameter>(const FString& InDefinitionName)
	{
		if (const TSharedPtr<OpenAPI::V2::FParameter>* FoundDefinition = InputSchema->Parameters->Find(InDefinitionName))
		{
			return *FoundDefinition;
		}

		return nullptr;
	}

	FWebAPINameVariant FWebAPISwaggerSchemaConverter::ResolvePropertyName(const TObjectPtr<UWebAPIProperty>& InProperty, const FWebAPITypeNameVariant& InPotentialName, const TOptional<bool>& bInIsArray)
	{
		check(InProperty);

		const bool bIsArray = bInIsArray.Get(InProperty->bIsArray);

		// If it's an array, it may be called the generic "Values", try to find a better name
		if(bIsArray && InProperty->Name == ProviderSettings.GetDefaultArrayPropertyName())
		{
			if(InPotentialName.IsValid())
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(ProviderSettings.Pluralize(InPotentialName.ToString(true)), InProperty->Name.GetJsonName());
				return NameInfo;
			}
			
			if(InProperty->Type.HasTypeInfo() && !InProperty->Type.TypeInfo->bIsBuiltinType)
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(ProviderSettings.Pluralize(InProperty->Type.ToString(true)), InProperty->Name.GetJsonName());
				return NameInfo;
			}
		}
		else if(InProperty->Name == ProviderSettings.GetDefaultPropertyName())
		{
			if(InPotentialName.IsValid())
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(InPotentialName.ToString(true), InProperty->Name.GetJsonName());
				return NameInfo;
			}
			
			if(InProperty->Type.HasTypeInfo() && !InProperty->Type.TypeInfo->bIsBuiltinType)
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(InProperty->Type.ToString(true), InProperty->Name.GetJsonName());
				return NameInfo;
			}
		}

		return InProperty->Name;
	}

	template <typename ObjectType>
	TSharedPtr<ObjectType> FWebAPISwaggerSchemaConverter::ResolveReference(const Json::TJsonReference<ObjectType>& InJsonReference, FString& OutDefinitionName, bool bInCheck)
	{
		if (InJsonReference.IsSet())
		{
			return InJsonReference.GetShared();
		}

		if(!InJsonReference.IsValid())
		{
			return nullptr;
		}
		
		FString DefinitionName = InJsonReference.GetLastPathSegment();
		if (TSharedPtr<ObjectType> FoundDefinition = ResolveReference<ObjectType>(DefinitionName))
		{
			OutDefinitionName = DefinitionName; // Only set if found
			return FoundDefinition;
		}

		if(bInCheck)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ReferencePath"), FText::FromString(InJsonReference.GetPath()));
			MessageLog->LogWarning(FText::Format(LOCTEXT("CannotResolveJsonReference", "Couldn't resolve JsonReference \"{ReferencePath}\""), Args),	FWebAPISwaggerProvider::LogName);
		}

		return nullptr;
	}

	template <>
	bool FWebAPISwaggerSchemaConverter::IsArray(const TSharedPtr<UE::WebAPI::OpenAPI::V2::FSchema>& InSchema)
	{
		return InSchema->Type.Get(TEXT(""))	== TEXT("array");
	}

	template <>
	bool FWebAPISwaggerSchemaConverter::IsArray(const TSharedPtr<UE::WebAPI::OpenAPI::V2::FParameter>& InSchema)
	{
		return InSchema->Type.Get(InSchema->Schema.IsSet()
								? InSchema->Schema.GetValue().GetShared()->Type.Get(TEXT(""))
								: TEXT(""))	== TEXT("array");
	}

	template <typename SchemaType, typename ModelType>
	bool FWebAPISwaggerSchemaConverter::ConvertModelBase(const TSharedPtr<SchemaType>& InSchema,
														ModelType* OutModel)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

		OutModel->Description = InSchema->Description.Get(TEXT(""));

		SET_OPTIONAL(InSchema->bRequired, OutModel->bIsRequired)
		
		SET_OPTIONAL_FLAGGED(InSchema->Minimum, OutModel->MinimumValue, OutModel->bUseMinimumValue);
		SET_OPTIONAL_FLAGGED(InSchema->Maximum, OutModel->MaximumValue, OutModel->bUseMaximumValue);

		if(!OutModel->bUseMinimumValue)
		{
			SET_OPTIONAL_FLAGGED(InSchema->MinLength, OutModel->MinimumValue, OutModel->bUseMinimumValue);
		}

		if(!OutModel->bUseMaximumValue)
		{
			SET_OPTIONAL_FLAGGED(InSchema->MaxLength, OutModel->MaximumValue, OutModel->bUseMaximumValue);					
		}
		
		SET_OPTIONAL_FLAGGED(InSchema->Pattern, OutModel->Pattern, OutModel->bUsePattern);
		
		return true;
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPIEnum> FWebAPISwaggerSchemaConverter::ConvertEnum(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InEnumTypeName) const
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

		FWebAPITypeNameVariant EnumTypeName;
		if(InEnumTypeName.IsValid())
		{
			EnumTypeName = InEnumTypeName;
		}
		else
		{
			const FString EnumName = InSrcSchema->Name;
			check(!EnumName.IsEmpty());

			const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
			EnumTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(EnumName),
				EnumName,
				StaticTypeRegistry->Enum);
		}

		const TObjectPtr<UWebAPIEnum>& DstEnum = OutputSchema->AddEnum(EnumTypeName.TypeInfo.Get());
		DstEnum->Name = EnumTypeName;
		DstEnum->Description = InSrcSchema->Description.Get(TEXT(""));

		UWebAPIModelBase* ModelBase = Cast<UWebAPIModelBase>(DstEnum);
		if (!ConvertModelBase(InSrcSchema, ModelBase))
		{
			return nullptr;
		}

		for (const FString& SrcEnumValue : InSrcSchema->Enum.GetValue())
		{
			const TObjectPtr<UWebAPIEnumValue> DstEnumValue = DstEnum->AddValue();
			DstEnumValue->Name.NameInfo.Name = NameTransformer(SrcEnumValue);
			DstEnumValue->Name.NameInfo.JsonName = SrcEnumValue;
		}

		DstEnum->BindToTypeInfo();
		
		return DstEnum;
	}

	template <>
	bool FWebAPISwaggerSchemaConverter::PatchProperty<OpenAPI::V2::FParameter>(
		const FWebAPITypeNameVariant& InModelName,
		const FWebAPINameVariant& InPropertyName,
		const TSharedPtr<OpenAPI::V2::FParameter>& InSchema,
		const FString& InDefinitionName,
		UWebAPIProperty* OutProperty)
	{
		const FString DefinitionName = !InDefinitionName.IsEmpty()
										? InDefinitionName
										: ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName);

		OutProperty->Type = ResolveType(InSchema, DefinitionName);

		UWebAPIModelBase* ModelBase = OutProperty;
		if (!ConvertModelBase(InSchema, ModelBase))
		{
			return false;
		}

		OutProperty->bIsArray = IsArray(InSchema);
		
		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);				
		OutProperty->Name = ResolvePropertyName(OutProperty, InModelName, {});
		OutProperty->bIsRequired = InSchema->bRequired.Get(false);
		OutProperty->BindToTypeInfo();

		// Add enum as it's own model, and reference it as this properties type
		if (InSchema->Enum.IsSet() && !InSchema->Enum->IsEmpty())
		{
			const FWebAPITypeNameVariant EnumTypeName = OutProperty->Type;
			EnumTypeName.TypeInfo->SetNested(InModelName);
			FString EnumName = OutProperty->Name.ToString(true);

			// Only make nested name if the model and property name aren't the same, otherwise you get "NameName"!
			if(EnumName != InModelName)
			{
				EnumName = ProviderSettings.MakeNestedPropertyTypeName(InModelName, OutProperty->Name.ToString(true));
			}
			EnumTypeName.TypeInfo->SetName(EnumName);

			const UWebAPIEnum* Enum = ConvertEnum(InSchema, EnumTypeName);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitEnumForPropertyOfModel", "Implicit enum created for property \"{PropertyName}\" of model \"{ModelName}\"."),
				TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
				TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
			MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);
			
			Enum->Name.TypeInfo->DebugString += LogMessage.ToString();
			Enum->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();

			OutProperty->Type = Enum->Name;
			OutProperty->Type.TypeInfo->SetModel(Enum);
		}
		// Add struct as it's own model, and reference it as this properties type
		else if (OutProperty->Type.ToString(true).IsEmpty())
		{
			UE_LOGF(LogWebAPIOpenAPI, Error, "Shouldn't hit this!");
		}

		return true;
	}

	bool FWebAPISwaggerSchemaConverter::PatchProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const FWebAPITypeNameVariant& InPropertyTypeName, UWebAPIProperty* OutProperty)
	{
		OutProperty->Type = InPropertyTypeName;

		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);
		OutProperty->Name = ResolvePropertyName(OutProperty, OutProperty->Type.ToString(true), {});
		OutProperty->bIsRequired = false;
		OutProperty->BindToTypeInfo();

		return true;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertOperationParameter(
		const FWebAPINameVariant& InParameterName,
		const TSharedPtr<OpenAPI::V2::FParameter>& InParameter,
		const FString& InDefinitionName,
		UWebAPIOperationParameter* OutParameter)
	{
		// Will get schema or create if it doesn't exist (but will be empty)
		Json::TJsonReference<OpenAPI::V2::FSchema> ParameterSchema = InParameter->Schema.Get({});

		static TMap<FString, EWebAPIParameterStorage> InToStorage = {
			{TEXT("query"), EWebAPIParameterStorage::Query},
			{TEXT("header"), EWebAPIParameterStorage::Header},
			{TEXT("path"), EWebAPIParameterStorage::Path},
			{TEXT("formData"), EWebAPIParameterStorage::Body},
			{TEXT("body"), EWebAPIParameterStorage::Body}
		};

		OutParameter->Storage = InToStorage[InParameter->In];
		
		if (ParameterSchema.IsSet())
		{
			OutParameter->Type = ResolveType(ParameterSchema.GetShared(), InDefinitionName);
		}
		else
		{
			OutParameter->Type = ResolveType(InParameter, InDefinitionName, GetDefaultJsonTypeForStorage(OutParameter->Storage));
		}

		OutParameter->bIsArray = IsArray(InParameter);
		if (!OutParameter->Type.HasTypeInfo())
		{
			OutParameter->Type = ResolveType(InParameter->Type.Get(GetDefaultJsonTypeForStorage(OutParameter->Storage)),
				InParameter->Format.Get(TEXT("")),
				InDefinitionName,
				ParameterSchema.GetShared());
		}

		UWebAPIModelBase* ModelBase = Cast<UWebAPIModelBase>(OutParameter);
		if (!ConvertModelBase(InParameter, ModelBase))
		{
			return false;
		}

		OutParameter->Name = FWebAPINameInfo(InParameterName.ToString(true), InParameterName.GetJsonName(), OutParameter->Type);

		// Add struct as it's own model, and reference it as this properties type
		if (OutParameter->Type.ToString(true).IsEmpty())
		{
			const FWebAPITypeNameVariant ModelTypeName = OutParameter->Type;
			ModelTypeName.TypeInfo->SetName(InDefinitionName);
			ModelTypeName.TypeInfo->Prefix = TEXT("F");

			UWebAPIModel* Model = OutputSchema->AddModel<UWebAPIModel>(
				ModelTypeName.HasTypeInfo() ? ModelTypeName.TypeInfo.Get() : nullptr);
			PatchModel(InParameter->Schema.GetValue().GetShared(), {}, Model);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitModelForParameterOfOperation", "Implicit model created for parameter \"{ParameterName}\" of operation \"{OperationName}\"."),
				TEXT("ParameterName"), FText::FromString(*InParameterName.ToString(true)),
				TEXT("OperationName"), FText::FromString(TEXT("?")));
			MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);
			
			Model->Name.TypeInfo->DebugString += LogMessage.ToString();
			
			Model->Name.TypeInfo->JsonName = InParameterName.GetJsonName();
			Model->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;

			OutParameter->Type = Model->Name;
			OutParameter->Type.TypeInfo->SetModel(Model);
		}

		// Special case for "body" parameters
		if (InParameter->In == TEXT("body"))
		{
			if(OutParameter->Name.ToString(true) == TEXT("Body"))
			{
				OutParameter->Name = InDefinitionName;
			}

			if (OutParameter->bIsArray)
			{
				FString Name = OutParameter->Name.ToString(true);
				Name = ProviderSettings.Singularize(Name);
				Name = ProviderSettings.Pluralize(Name);
				OutParameter->Name = Name;
			}
		}

		OutParameter->Description = InParameter->Description.Get(TEXT(""));
		OutParameter->bIsRequired = InParameter->bRequired.Get(false);
		OutParameter->BindToTypeInfo();

		return true;
	}

	TObjectPtr<UWebAPIParameter> FWebAPISwaggerSchemaConverter::ConvertParameter(const TSharedPtr<OpenAPI::V2::FParameter>& InSrcParameter)
	{
		const FString* ParameterName = InputSchema->Parameters->FindKey(InSrcParameter);
		check(ParameterName);

		const FString ParameterJsonName = InSrcParameter->Name.IsEmpty() ? *ParameterName : InSrcParameter->Name;

		const FWebAPITypeNameVariant ParameterTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Parameter,
			ProviderSettings.MakeParameterTypeName(NameTransformer(*ParameterName)),
			ParameterJsonName,
			TEXT("F"));
		
		ParameterTypeName.TypeInfo->Suffix = TEXT("Parameter");

		// Don't use ParameterTypeName->Name, it might have a Parameter specific pre/postfix.
		FString SrcParameterDefinitionName = NameTransformer(*ParameterName);

		// Will get schema or create if it doesn't exist (but will be empty)
		const TSharedPtr<OpenAPI::V2::FSchema> SrcParameterSchema = ResolveReference(InSrcParameter->Schema.Get({}), SrcParameterDefinitionName, false);

		UWebAPIParameter* DstParameter = OutputSchema->AddParameter(ParameterTypeName.TypeInfo.Get());

		static TMap<FString, EWebAPIParameterStorage> InToStorage = {
			{TEXT("query"), EWebAPIParameterStorage::Query},
			{TEXT("header"), EWebAPIParameterStorage::Header},
			{TEXT("path"), EWebAPIParameterStorage::Path},
			{TEXT("formData"), EWebAPIParameterStorage::Body},
			{TEXT("body"), EWebAPIParameterStorage::Body}
		};

		DstParameter->Storage = InToStorage[InSrcParameter->In];

		DstParameter->bIsArray = IsArray(InSrcParameter);
		if (!DstParameter->Type.HasTypeInfo())
		{
			DstParameter->Type = ResolveType(InSrcParameter->Type.Get(GetDefaultJsonTypeForStorage(DstParameter->Storage)),
				InSrcParameter->Format.Get(TEXT("")),
				SrcParameterDefinitionName,
				SrcParameterSchema);
		}

		UWebAPIModel* Model = Cast<UWebAPIModel>(DstParameter);
		if (!PatchModel(InSrcParameter, ParameterTypeName, Model))
		{
			return nullptr;
		}

		// Set the (single) property, will either be a single or array of a model or enum, based on the parameter or it's schema + parameter
		{
			FString PropertyName = DstParameter->bIsArray ? ProviderSettings.GetDefaultArrayPropertyName() : ProviderSettings.GetDefaultPropertyName();
			SrcParameterDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(SrcParameterDefinitionName, PropertyName);
			
			TObjectPtr<UWebAPIProperty>& DstProperty = DstParameter->Property = Model->Properties.Add_GetRef(NewObject<UWebAPIProperty>(Model));
			PatchProperty(Model->Name,
				PropertyName,
				InSrcParameter,
				SrcParameterDefinitionName,
				DstProperty);

			// Add enum as it's own model, and reference it as this properties type
			if (SrcParameterSchema.IsValid() && SrcParameterSchema->Enum.IsSet() && !SrcParameterSchema->Enum->IsEmpty())
			{
				const FWebAPITypeNameVariant EnumTypeName = DstParameter->Type;
				EnumTypeName.TypeInfo->SetName(ProviderSettings.MakeParameterTypeName(*ParameterName));
				EnumTypeName.TypeInfo->SetNested(ParameterTypeName);

				const UWebAPIEnum* Enum = ConvertEnum(SrcParameterSchema, EnumTypeName);

				const FText LogMessage = FText::FormatNamed(
					LOCTEXT("AddedImplicitEnumForParameter", "Implicit enum created for parameter \"{Name}\"."),
					TEXT("Name"), FText::FromString(**ParameterName));
				MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);

				Enum->Name.TypeInfo->DebugString += LogMessage.ToString();

				DstParameter->Type = Enum->Name;
				DstParameter->Type.TypeInfo->SetModel(Enum);
			}
			// Add struct as it's own model, and reference it as this properties type
			else if (DstParameter->Type.ToString(true).IsEmpty())
			{
				const FWebAPITypeNameVariant ModelTypeName = DstParameter->Name;
				ModelTypeName.TypeInfo->SetName(SrcParameterDefinitionName);
				ModelTypeName.TypeInfo->Prefix = TEXT("F");
				ModelTypeName.TypeInfo->SetNested(ParameterTypeName);

				PatchModel(InSrcParameter, {}, DstParameter);

				const FText LogMessage = FText::FormatNamed(
					LOCTEXT("AddedImplicitModelForParameter", "Implicit model created for parameter \"{ParameterName}\"."),
					TEXT("ParameterName"), FText::FromString(**ParameterName));
				MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);
				
				DstParameter->Name.TypeInfo->DebugString += LogMessage.ToString();
				
				DstParameter->Name.TypeInfo->JsonName = ParameterTypeName.GetJsonName();
				DstParameter->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
			}
		}

		// Special case for "body" parameters
		if (InSrcParameter->In == TEXT("body"))
		{
			DstParameter->Name = SrcParameterDefinitionName;

			if (DstParameter->bIsArray)
			{
				DstParameter->Name = ProviderSettings.Pluralize(SrcParameterDefinitionName);
			}
		}

		DstParameter->BindToTypeInfo();

		DstParameter->Description = InSrcParameter->Description.Get(TEXT(""));
		DstParameter->bIsRequired = InSrcParameter->bRequired.Get(false);
	
		return DstParameter;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertRequest(const FWebAPITypeNameVariant& InOperationName, const TSharedPtr<OpenAPI::V2::FOperation>& InOperation, UWebAPIOperationRequest* OutRequest)
	{
		const FWebAPITypeNameVariant RequestTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Model,
			ProviderSettings.MakeRequestTypeName(InOperationName),
			InOperationName.ToString(true),
			IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object);
#if WITH_WEBAPI_DEBUG
		RequestTypeName.TypeInfo->DebugString += TEXT(">ConvertRequest");
#endif
		RequestTypeName.TypeInfo->Prefix = TEXT("F");

		OutRequest->Name = RequestTypeName;

		if(InOperation->Parameters.IsSet())
		{
			for (Json::TJsonReference<OpenAPI::V2::FParameter>& SrcParameter : InOperation->Parameters.GetValue())
			{
				FString SrcParameterDefinitionName;
				SrcParameter.Set(ResolveReference(SrcParameter, SrcParameterDefinitionName));

				FWebAPINameVariant ParameterName = FWebAPINameInfo(NameTransformer(SrcParameter->Name), SrcParameter->Name);
				if(SrcParameterDefinitionName.IsEmpty())
				{
					SrcParameterDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(InOperationName, ParameterName);
				}

				// Will get schema or create if it doesn't exist (but will be empty)
				Json::TJsonReference<OpenAPI::V2::FSchema> SrcParameterSchema = SrcParameter->Schema.Get({});

				if (!SrcParameterSchema.GetPath().IsEmpty())
				{
					SrcParameterDefinitionName = SrcParameterSchema.GetLastPathSegment();
				}

				if (SrcParameterSchema.IsSet() && SrcParameterSchema->Items.IsSet()	&& !SrcParameterSchema->Items->GetPath().IsEmpty())
				{
					SrcParameterDefinitionName = SrcParameterSchema->Items->GetLastPathSegment();
				}

				UWebAPIOperationParameter* DstParameter = OutRequest->Parameters.Add_GetRef(NewObject<UWebAPIOperationParameter>(OutRequest));
				ConvertOperationParameter(ParameterName, SrcParameter.GetShared(), SrcParameterDefinitionName, DstParameter);
				DstParameter->BindToTypeInfo();
			
				// Special case - if the name is "body", set as Body property and not Parameter
				if (SrcParameter->Name.Equals(TEXT("body"), ESearchCase::IgnoreCase) && !SrcParameterDefinitionName.IsEmpty())
				{
					// Check for existing definition
					if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, SrcParameterDefinitionName))
					{
						DstParameter->Model = Cast<UWebAPIModel>((*FoundGeneratedType)->GetModel());
						return true;
					}
				}
			}
		}
		// Operation has no parameters, but if arbitrary json is enabled, make a Value property of type JsonObject
		else if(ProviderSettings.bEnableArbitraryJsonPayloads)
		{
			FWebAPINameVariant ParameterName = TEXT("Value");

			const TObjectPtr<UWebAPIOperationParameter> DstParameter = OutRequest->Parameters.Add_GetRef(NewObject<UWebAPIOperationParameter>(OutRequest));
			DstParameter->Type = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->JsonObject;
			DstParameter->Name = FWebAPINameInfo(ParameterName.ToString(true), ParameterName.GetJsonName(), DstParameter->Type);
			DstParameter->bIsMixin = true; // treat as object, not as field
			DstParameter->Storage = EWebAPIParameterStorage::Body;
			DstParameter->BindToTypeInfo();
		}

		OutRequest->BindToTypeInfo();
		
		return true;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertResponse(const FWebAPITypeNameVariant& InOperationName, uint32 InResponseCode, const TSharedPtr<OpenAPI::V2::FResponse>& InResponse, UWebAPIOperationResponse* OutResponse)
	{
		const FWebAPITypeNameVariant ResponseTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Model,
			ProviderSettings.MakeResponseTypeName(InOperationName, InResponseCode),
			InOperationName.ToString(true),
			IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object);
#if WITH_WEBAPI_DEBUG
		ResponseTypeName.TypeInfo->DebugString += TEXT(">ConvertResponse");
#endif
		ResponseTypeName.TypeInfo->Prefix = TEXT("F");

		check(ResponseTypeName.HasTypeInfo());

		OutResponse->Name = ResponseTypeName;
		OutResponse->Code = InResponseCode;
		OutResponse->Description = InResponse->Description;
		OutResponse->Message = OutResponse->Description;

		if (InResponse->Schema.IsSet())
		{
			// Will get schema or create if it doesn't exist (but will be empty)
			Json::TJsonReference<OpenAPI::V2::FSchema> SrcResponseSchema = InResponse->Schema.Get({});
			
			FString SrcPropertyDefinitionName;
			SrcResponseSchema.Set(ResolveReference(SrcResponseSchema, SrcPropertyDefinitionName));

			const FString PropertyName = SrcResponseSchema->Type.Get(GetDefaultJsonTypeForStorage(OutResponse->Storage)) == TEXT("array") ? ProviderSettings.GetDefaultArrayPropertyName() : ProviderSettings.GetDefaultPropertyName();
			UWebAPIProperty* DstProperty = OutResponse->Properties.Add_GetRef(NewObject<UWebAPIProperty>(OutResponse));
			DstProperty->bIsMixin = true;
			PatchProperty(OutResponse->Name,
				PropertyName,
				SrcResponseSchema.GetShared(),
				SrcPropertyDefinitionName,
				DstProperty);
		}

		OutResponse->BindToTypeInfo();

		return true;
	}

	TObjectPtr<UWebAPIOperation> FWebAPISwaggerSchemaConverter::ConvertOperation(const FString& InPath, const FString& InVerb, const TSharedPtr<OpenAPI::V2::FOperation>& InSrcOperation, const FWebAPITypeNameVariant& InOperationTypeName)
	{
		FString OperationName = InSrcOperation->OperationId.Get(InSrcOperation->Summary.Get(TEXT("")));
		OperationName = NameTransformer(OperationName);
		OperationName = ProviderSettings.MakeValidMemberName(OperationName, InVerb);

		check(!OperationName.IsEmpty());
		
		FWebAPITypeNameVariant OperationTypeName;
		if(InOperationTypeName.IsValid())
		{
			OperationName = InOperationTypeName.ToString(true);
			OperationTypeName = InOperationTypeName;
		}
		else
		{
			OperationTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Operation,
				NameTransformer(*OperationName),
				*OperationName,
				TEXT("U"));
			OperationTypeName.TypeInfo->SetName(OperationName);

#if WITH_WEBAPI_DEBUG
			OperationTypeName.TypeInfo->DebugString += TEXT(">ConvertOperation");
#endif
		}

		// A spec can have no tags, so ensure there's a default
		const FString FirstTag = ProviderSettings.ToPascalCase(InSrcOperation->Tags.Get({TEXT("Default")})[0]);
		TObjectPtr<UWebAPIService>* Service = OutputSchema->Services.Find(FirstTag);
		checkf(Service, TEXT("An operation must belong to a service!"));

		const TObjectPtr<UWebAPIOperation> DstOperation = (*Service)->Operations.Add_GetRef(NewObject<UWebAPIOperation>(*Service));
		DstOperation->Service = *Service;
		DstOperation->Verb = InVerb;
		DstOperation->Path = InPath;

		DstOperation->Name = OperationTypeName;
		// Choose first non-empty of: description, summary, operation id
		DstOperation->Description = InSrcOperation->Description.Get(
			InSrcOperation->OperationId.Get(InSrcOperation->Summary.Get(
				InSrcOperation->OperationId.Get(TEXT("")))));
		DstOperation->bIsDeprecated = InSrcOperation->bDeprecated.Get(false);

		ConvertRequest(DstOperation->Name, InSrcOperation, DstOperation->Request);

		if (InSrcOperation->Consumes.IsSet())
		{
			DstOperation->RequestContentTypes.Append(InSrcOperation->Consumes.GetValue());
		}

		if (InSrcOperation->Produces.IsSet())
		{
			DstOperation->ResponseContentTypes.Append(InSrcOperation->Produces.GetValue());
		}

		if (ensure(!InSrcOperation->Responses.IsEmpty()))
		{
			for (TPair<FString, TSharedPtr<OpenAPI::V2::FResponse>>& SrcResponse : InSrcOperation->Responses)
			{
				// If "Default", resolves to 0, and means all other unhandled codes (similar to default in switch statement)
				const uint32 Code = FCString::Atoi(*SrcResponse.Key);
				const TObjectPtr<UWebAPIOperationResponse> DstResponse = DstOperation->Responses.Add_GetRef(NewObject<UWebAPIOperationResponse>(DstOperation));
				ConvertResponse(DstOperation->Name, Code, SrcResponse.Value, DstResponse);

				// If success response (code 200), had no resolved properties but the operation says it returns something, then add that something as a property
				if(DstResponse->Properties.IsEmpty() && !DstOperation->ResponseContentTypes.IsEmpty() && DstResponse->Code == 200)
				{
					const FString PropertyName = ProviderSettings.GetDefaultPropertyName();
					const TObjectPtr<UWebAPIProperty>& DstProperty = DstResponse->Properties.Add_GetRef(NewObject<UWebAPIProperty>(DstResponse));
					DstProperty->bIsMixin = true;
					PatchProperty(DstResponse->Name,
						PropertyName,
						GetTypeForContentType(DstOperation->ResponseContentTypes[0]),
						DstProperty);
					DstProperty->Name = PropertyName;
					DstResponse->Storage = EWebAPIResponseStorage::Body;
				}
			}
		}

		check(DstOperation->Name.HasTypeInfo());

		const FName OperationObjectName = ProviderSettings.MakeOperationObjectName(*Service, OperationName);
		DstOperation->Rename(*OperationObjectName.ToString(), DstOperation->GetOuter());
		
		return DstOperation;
	}

	TObjectPtr<UWebAPIService> FWebAPISwaggerSchemaConverter::ConvertService(const FWebAPINameVariant& InName) const
	{
		const FString TagName = NameTransformer(InName);
		return OutputSchema->GetOrMakeService(TagName);
	}

	TObjectPtr<UWebAPIService> FWebAPISwaggerSchemaConverter::ConvertService(const TSharedPtr<OpenAPI::V2::FTag>& InTag) const
	{
		const TObjectPtr<UWebAPIService> Service = ConvertService(InTag->Name);
		Service->Description = InTag->Description.Get(TEXT(""));
		return Service;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertModels(const TMap<FString, TSharedPtr<OpenAPI::V2::FSchema>>& InSchemas,	UWebAPISchema* OutSchema)
	{
		bool bAllConverted = true;
		for (const TTuple<FString, TSharedPtr<OpenAPI::V2::FSchema>>& Item : InSchemas)
		{
			const FString& Name = Item.Key;
			const TSharedPtr<OpenAPI::V2::FSchema>& Schema = Item.Value;

			if (!Schema.IsValid())
			{
				bAllConverted = false;
				FFormatNamedArguments Args;
				Args.Add(TEXT("ModelName"), FText::FromString(Name));
				MessageLog->LogWarning(
					FText::Format(LOCTEXT("SchemaInvalid", "The schema for model \"{ModelName}\" was invalid/null."), Args),
					FWebAPISwaggerProvider::LogName);
				continue;
			}

			FWebAPITypeNameVariant ModelName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(Name),
				Name,
				TEXT("F"));

			UWebAPIModel* Model = OutSchema->AddModel<UWebAPIModel>(ModelName.TypeInfo.Get());
			PatchModel(Schema, {}, Model);
		}

		// Set Model property of TypeInfos where applicable
		for (const TObjectPtr<UWebAPIModelBase>& ModelBase : OutputSchema->Models)
		{
			if (const UWebAPIModel* Model = Cast<UWebAPIModel>(ModelBase))
			{
				Model->Name.TypeInfo->SetModel(Model);
			}
			else if (const UWebAPIEnum* Enum = Cast<UWebAPIEnum>(ModelBase))
			{
				Enum->Name.TypeInfo->SetModel(Enum);
			}
			else if (const UWebAPIParameter* ParameterModel = Cast<UWebAPIParameter>(ModelBase))
			{
				ParameterModel->Type.TypeInfo->SetModel(ParameterModel);
			}
		}

		return bAllConverted;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertParameters(const TMap<FString, TSharedPtr<OpenAPI::V2::FParameter>>& InParameters, UWebAPISchema* OutSchema)
	{
		bool bAllConverted = true;
		for (const TTuple<FString, TSharedPtr<OpenAPI::V2::FParameter>>& Item : InParameters)
		{
			const FWebAPITypeNameVariant& Name = Item.Key;
			const TSharedPtr<OpenAPI::V2::FParameter>& SrcParameter = Item.Value;

			bAllConverted &= ConvertParameter(SrcParameter) != nullptr;
		}

		// Re-bind models to their TypeInfos
		for (const TObjectPtr<UWebAPIModelBase>& Model : OutputSchema->Models)
		{
			Model->BindToTypeInfo();
		}

		return bAllConverted;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertSecurity(const UE::WebAPI::OpenAPI::V2::FSwagger& InSwagger, UWebAPISchema* OutSchema)
	{
		if (InSwagger.SecurityDefinitions.IsSet())
		{
			for (const TTuple<FString, TSharedPtr<OpenAPI::V2::FSecurityScheme>>& Item : InSwagger.
				SecurityDefinitions.GetValue())
			{
				const FString& Name = Item.Key;
				const TSharedPtr<OpenAPI::V2::FSecurityScheme>& SecuritySchema = Item.Value;
			}

			return true;
		}

		return false;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertTags(const TArray<TSharedPtr<OpenAPI::V2::FTag>>& InTags, UWebAPISchema* OutSchema) const
	{
		for (const TSharedPtr<OpenAPI::V2::FTag>& Tag : InTags)
		{
			ConvertService(Tag);
		}

		return OutSchema->Services.Num() > 0;
	}

	bool FWebAPISwaggerSchemaConverter::ConvertPaths(const TMap<FString, TSharedPtr<OpenAPI::V2::FPath>>& InPaths, UWebAPISchema* OutSchema)
	{
		for (const TTuple<FString, TSharedPtr<OpenAPI::V2::FPath>>& Item : InPaths)
		{
			const FString& Url = Item.Key;
			const TSharedPtr<OpenAPI::V2::FPath>& Path = Item.Value;

			TMap<FString, TSharedPtr<OpenAPI::V2::FOperation>> SrcVerbs;
			if (Path->Get.IsValid())
			{
				SrcVerbs.Add(TEXT("Get"), Path->Get);
			}
			if (Path->Put.IsValid())
			{
				SrcVerbs.Add(TEXT("Put"), Path->Put);
			}
			if (Path->Post.IsValid())
			{
				SrcVerbs.Add(TEXT("Post"), Path->Post);
			}
			if (Path->Delete.IsValid())
			{
				SrcVerbs.Add(TEXT("Delete"), Path->Delete);
			}
			if (Path->Options.IsValid())
			{
				SrcVerbs.Add(TEXT("Options"), Path->Options);
			}
			if (Path->Head.IsValid())
			{
				SrcVerbs.Add(TEXT("Head"), Path->Head);
			}
			if (Path->Patch.IsValid())
			{
				SrcVerbs.Add(TEXT("Patch"), Path->Patch);
			}

			// Each path can have multiple, ie. Get, Put, Delete
			for (const TPair<FString, TSharedPtr<OpenAPI::V2::FOperation>>& VerbOperationPair : SrcVerbs)
			{
				FString Verb = VerbOperationPair.Key;
				if (VerbOperationPair.Value != nullptr
					&& VerbOperationPair.Value.IsValid())
				{
					TSharedPtr<OpenAPI::V2::FOperation> SrcOperation = VerbOperationPair.Value;

					TArray<FString> Tags;
					if (SrcOperation.IsValid() && SrcOperation->Tags.IsSet())
					{
						Tags = SrcOperation->Tags.GetValue();
					}
					else
					{
						Tags = { TEXT("Default") };
					}

					for (FString& Tag : Tags)
					{
						// Do first to retain original case
						TObjectPtr<UWebAPIService> Service = ConvertService(Tag);
						Tag = NameTransformer(Tag);

						FString OperationName = SrcOperation->OperationId.Get(Verb + Tag);
						OperationName = NameTransformer(OperationName);
						OperationName = ProviderSettings.MakeValidMemberName(OperationName, Verb);

						FString OperationNamePrefix = TEXT("");
						int32 OperationNameSuffix = 0;

						// Check if this already exists - each operation is unique so the name needs to be different - prepend verb
						while(OutSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Operation,
							OperationNamePrefix + OperationName + (OperationNameSuffix > 0 ? FString::FormatAsNumber(OperationNameSuffix) : TEXT(""))) != nullptr)
						{
							if(!OperationName.StartsWith(Verb))
							{
								OperationNamePrefix = Verb;
							}
							OperationNameSuffix += 1; 
						}

						OperationName = OperationNamePrefix + OperationName + (OperationNameSuffix > 0 ? FString::FormatAsNumber(OperationNameSuffix) : TEXT(""));
		
						TObjectPtr<UWebAPITypeInfo> OperationTypeInfo = OutSchema->TypeRegistry->
							GetOrMakeGeneratedType(
								EWebAPISchemaType::Operation,
								OperationName,
								OperationName,
								TEXT("U"));

						const TObjectPtr<UWebAPIOperation> Operation = ConvertOperation(Url, Verb, SrcOperation, OperationTypeInfo);
						Operation->Service = Service;
						Operation->Verb = Verb;
						Operation->Path = Url;

						FStringFormatNamedArguments FormatArgs;
						FormatArgs.Add(TEXT("ClassName"), UWebAPIOperation::StaticClass()->GetName());
						FormatArgs.Add(TEXT("ServiceName"), Service->Name.ToString(true));
						FormatArgs.Add(TEXT("OperationName"), OperationName);

						const FName OperationObjectName = ProviderSettings.MakeOperationObjectName(Service, OperationName);
						Operation->Rename(*OperationObjectName.ToString(), Operation->GetOuter());

						Operation->BindToTypeInfo();
					}
				}
			}
		}
		return true;
	}
};

#undef SET_OPTIONAL_FLAGGED
#undef SET_OPTIONAL

#undef LOCTEXT_NAMESPACE
