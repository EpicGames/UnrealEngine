// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "WebAPIProviderSettings.h"

class FWebAPIMessageLog;
class UWebAPIEnum;
class UWebAPIModel;
class UWebAPIOperation;
class UWebAPIOperationParameter;
class UWebAPIOperationRequest;
class UWebAPIOperationResponse;
class UWebAPIParameter;
class UWebAPIProperty;
class UWebAPISchema;
class UWebAPITypeInfo;
namespace UE::Json { template <class ObjectType> class TJsonReference; }
namespace UE::WebAPI::OpenAPI::V2 { class FOperation; }
namespace UE::WebAPI::OpenAPI::V2 { class FParameter; }
namespace UE::WebAPI::OpenAPI::V2 { class FPath; }
namespace UE::WebAPI::OpenAPI::V2 { class FResponse; }
namespace UE::WebAPI::OpenAPI::V2 { class FSchema; }
namespace UE::WebAPI::OpenAPI::V2 { class FSwagger; }
namespace UE::WebAPI::OpenAPI::V2 { class FTag; }

namespace UE::WebAPI::Swagger
{
	class FWebAPISwaggerSchemaConverter
	{
	public:
		FWebAPISwaggerSchemaConverter(
			const TSharedPtr<const UE::WebAPI::OpenAPI::V2::FSwagger>& InSwagger,
			UWebAPISchema* InWebAPISchema,
			const TSharedRef<FWebAPIMessageLog>& InMessageLog,
			const FWebAPIProviderSettings& InProviderSettings);

		bool Convert();

	public:
		FString NameTransformer(const FWebAPINameVariant& InString) const;

		TObjectPtr<UWebAPITypeInfo> ResolveType(FString InJsonType, FString InTypeHint, FString InDefinitionName, const TSharedPtr<OpenAPI::V2::FSchema>& InSchema = nullptr) const;

		template <typename SchemaType>
		TObjectPtr<UWebAPITypeInfo> ResolveType(const TSharedPtr<SchemaType>& InSchema,	const FString& InDefinitionName = {}, FString InJsonType = {});

		TObjectPtr<UWebAPITypeInfo> GetTypeForContentType(const FString& InContentType);

		template <typename StorageType>
		FString GetDefaultJsonTypeForStorage(const StorageType& InStorage);

		template <typename ObjectType>
		TSharedPtr<ObjectType> ResolveReference(const FString& InDefinitionName);

		/** Only resolves if necessary. */
		template <typename ObjectType>
		TSharedPtr<ObjectType> ResolveReference(const Json::TJsonReference<ObjectType>& InJsonReference, FString& OutDefinitionName, bool bInCheck = true);

		//FWebAPINameVariant ResolveModelName(const TObjectPtr<UWebAPIProperty>& InProperty, const TOptional<bool>& bInIsArray, const FWebAPITypeNameVariant& InPossibleName = {});

		FWebAPINameVariant ResolvePropertyName(const TObjectPtr<UWebAPIProperty>& InProperty, const FWebAPITypeNameVariant& InPotentialName, const TOptional<bool>& bInIsArray = {});

		template <typename SchemaType>
		bool IsArray(const TSharedPtr<SchemaType>& InSchema);

		template <typename SchemaType, typename ModelType>
		static bool ConvertModelBase(const TSharedPtr<SchemaType>& InSchema, ModelType* OutModel);

		template <typename SchemaType>
		TObjectPtr<UWebAPIEnum> ConvertEnum(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InEnumTypeName = {}) const;

		/** Modifies an existing Property using the supplied source object. */
		template <typename SchemaType>
		bool PatchProperty(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPINameVariant& InPropertyName, UWebAPIProperty* OutProperty);

		/** Modifies an existing Property using the supplied source object. */
		template <typename SchemaType>
		bool PatchProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const TSharedPtr<SchemaType>& InSchema, const FString& InDefinitionName, UWebAPIProperty* OutProperty);

		/** Modifies an existing Property using the supplied TypeInfo. */
		bool PatchProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const FWebAPITypeNameVariant& InPropertyTypeName, UWebAPIProperty* OutProperty);
		
		/** Creates a new Property from the supplied source object. */
		template <typename SchemaType>
		TObjectPtr<UWebAPIProperty> ConvertProperty(const TSharedPtr<SchemaType>& InSrcSchema, const TObjectPtr<UWebAPIModel>& InModel, const FWebAPINameVariant& InPropertyName = {}, const FString& InDefinitionName = {});

		/** Modifies an existing Model using the supplied source object. */
		template <typename SchemaType>
		bool PatchModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel);
		
		/** Creates a new Model from the supplied source object. */
		template <typename SchemaType>
		TObjectPtr<UWebAPIModel> ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName = {});

		bool ConvertOperationParameter(const FWebAPINameVariant& InParameterName, const TSharedPtr<OpenAPI::V2::FParameter>& InParameter, const FString& InDefinitionName, UWebAPIOperationParameter* OutParameter);

		TObjectPtr<UWebAPIParameter> ConvertParameter(const TSharedPtr<OpenAPI::V2::FParameter>& InSrcParameter);
		
		bool ConvertRequest(const FWebAPITypeNameVariant& InOperationName, const TSharedPtr<OpenAPI::V2::FOperation>& InOperation,  UWebAPIOperationRequest* OutRequest);

		bool ConvertResponse(const FWebAPITypeNameVariant& InOperationName, uint32 InResponseCode, const TSharedPtr<OpenAPI::V2::FResponse>& InResponse, UWebAPIOperationResponse* OutResponse);

		TObjectPtr<UWebAPIOperation> ConvertOperation(const FString& InPath, const FString& InVerb, const TSharedPtr<OpenAPI::V2::FOperation>& InSrcOperation, const FWebAPITypeNameVariant& InOperationTypeName = {});

		TObjectPtr<UWebAPIService> ConvertService(const FWebAPINameVariant& InName) const;
		TObjectPtr<UWebAPIService> ConvertService(const TSharedPtr<OpenAPI::V2::FTag>& InTag) const;

		bool ConvertModels(const TMap<FString, TSharedPtr<OpenAPI::V2::FSchema>>& InSchemas, UWebAPISchema* OutSchema);
		bool ConvertParameters(const TMap<FString, TSharedPtr<OpenAPI::V2::FParameter>>& InParameters, UWebAPISchema* OutSchema);

		bool ConvertSecurity(const UE::WebAPI::OpenAPI::V2::FSwagger& InSwagger, UWebAPISchema* OutSchema);

		bool ConvertTags(const TArray<TSharedPtr<OpenAPI::V2::FTag>>& InTags, UWebAPISchema* OutSchema) const;

		bool ConvertPaths(const TMap<FString, TSharedPtr<OpenAPI::V2::FPath>>& InPaths, UWebAPISchema* OutSchema);

	private:
		TSharedPtr<const UE::WebAPI::OpenAPI::V2::FSwagger> InputSchema;
		TStrongObjectPtr<UWebAPISchema> OutputSchema;
		TSharedPtr<FWebAPIMessageLog> MessageLog;
		FWebAPIProviderSettings ProviderSettings;
	};

	template <>
	TSharedPtr<OpenAPI::V2::FSchema> FWebAPISwaggerSchemaConverter::ResolveReference(const FString& InDefinitionName);

	template <>
	TSharedPtr<OpenAPI::V2::FParameter> FWebAPISwaggerSchemaConverter::ResolveReference(const FString& InDefinitionName);

	/** Modifies an existing Model using the supplied source object. */
	template <>
	bool FWebAPISwaggerSchemaConverter::PatchModel(const TSharedPtr<OpenAPI::V2::FParameter>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel);
}

#include "WebAPISwaggerConverter.inl"
