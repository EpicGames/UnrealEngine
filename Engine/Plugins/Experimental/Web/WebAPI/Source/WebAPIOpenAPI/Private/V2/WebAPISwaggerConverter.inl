// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIType.h"
#include "Dom/WebAPITypeRegistry.h"
#include "V2/WebAPISwaggerProvider.h"

#define LOCTEXT_NAMESPACE "WebAPISwaggerConverter"

namespace UE::WebAPI::Swagger
{
	template <typename SchemaType>
	bool FWebAPISwaggerSchemaConverter::PatchProperty(const FWebAPITypeNameVariant& InModelName,
														const FWebAPINameVariant& InPropertyName,
														const TSharedPtr<SchemaType>& InSchema,
														const FString& InDefinitionName,
														UWebAPIProperty* OutProperty)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

		FString DefinitionName = !InDefinitionName.IsEmpty()
								? InDefinitionName
								: ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName);

		TSharedPtr<UE::WebAPI::OpenAPI::V2::FSchema> ItemSchema = InSchema;
		if(InSchema->Items.IsSet())
		{
			if(!InSchema->Items->GetPath().IsEmpty())
			{
				ItemSchema = ResolveReference(InSchema->Items.GetValue(), DefinitionName);
				DefinitionName = InSchema->Items->GetLastPathSegment();						
			}
		}

		OutProperty->Type = ResolveType<SchemaType>(ItemSchema, DefinitionName);

		UWebAPIModelBase* ModelBase = OutProperty;
		if (!ConvertModelBase<SchemaType, UWebAPIModelBase>(ItemSchema, ModelBase))
		{
			return false;
		}

		OutProperty->bIsArray = IsArray(InSchema);
		
		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);
		OutProperty->Name = ResolvePropertyName(OutProperty, OutProperty->Type.ToString(true), {});
		OutProperty->bIsRequired = InSchema->bRequired.Get(false);
		OutProperty->BindToTypeInfo();

		// Add enum as it's own model, and reference it as this properties type
		if (InSchema->Enum.IsSet() && !InSchema->Enum->IsEmpty())
		{
			const FWebAPITypeNameVariant EnumTypeName = OutProperty->Type;
			EnumTypeName.TypeInfo->SetName(ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName));

			const UWebAPIEnum* Enum = ConvertEnum(InSchema, EnumTypeName);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitEnumForPropertyOfModel", "Implicit enum created for property \"{PropertyName}\" of model \"{ModelName}\"."),
				TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
				TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
			MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);

			Enum->Name.TypeInfo->DebugString += LogMessage.ToString();
			
			Enum->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();
			Enum->Name.TypeInfo->SetNested(InModelName);

			OutProperty->Type = Enum->Name;
			OutProperty->Type.TypeInfo->SetModel(Enum);
		}
		// Add struct as it's own model, and reference it as this properties type
		else if (OutProperty->Type.ToString(true).IsEmpty())
		{
			const FWebAPITypeNameVariant ModelTypeName = OutProperty->Type;
			ModelTypeName.TypeInfo->SetName(ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName));
			ModelTypeName.TypeInfo->Prefix = TEXT("F");
			ModelTypeName.TypeInfo->SetNested(InModelName);

			UWebAPIModel* Model = OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.HasTypeInfo() ? ModelTypeName.TypeInfo.Get() : nullptr);
			PatchModel(InSchema, {}, Model);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitModelForPropertyOfModel", "Implicit model created for property \"{PropertyName}\" of model \"{ModelName}\"."),
				TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
				TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
			MessageLog->LogInfo(LogMessage, FWebAPISwaggerProvider::LogName);
			
			Model->Name.TypeInfo->DebugString += LogMessage.ToString();
			
			Model->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();
			Model->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;

			OutProperty->Type = Model->Name;
			OutProperty->Type.TypeInfo->SetModel(Model);
		}
		
		OutProperty->Name = ResolvePropertyName(OutProperty, InModelName);

		return true;
	}

		template <typename SchemaType>
		TObjectPtr<UWebAPIProperty> FWebAPISwaggerSchemaConverter::ConvertProperty(const TSharedPtr<SchemaType>& InSrcSchema, const TObjectPtr<UWebAPIModel>& InModel, const FWebAPINameVariant& InPropertyName, const FString& InDefinitionName)
		{
			static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

			const FWebAPINameVariant SrcPropertyName = InPropertyName;
			FString SrcPropertyDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(*InModel->Name.ToString(true), SrcPropertyName);
			const TSharedPtr<OpenAPI::V2::FSchema> SrcPropertyValue = ResolveReference(InSrcSchema, SrcPropertyDefinitionName);

			if (SrcPropertyValue)
			{
				const TObjectPtr<UWebAPIProperty>& DstProperty = InModel->Properties.Add_GetRef(NewObject<UWebAPIProperty>(InModel));
				PatchProperty(InModel->Name,
					FWebAPINameInfo(NameTransformer(SrcPropertyName), SrcPropertyName.GetJsonName(), InModel->Name),
					SrcPropertyValue,
					SrcPropertyDefinitionName,
					DstProperty);

				return DstProperty;
			}

			return nullptr;
		}

		template <typename SchemaType>
		bool FWebAPISwaggerSchemaConverter::PatchModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel)
		{
			static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V2::FSchemaBase>::Value, "Type is not derived from OpenAPI::V2::FSchemaBase.");

			FWebAPITypeNameVariant ModelTypeName;
			if(InModelTypeName.IsValid())
			{
				ModelTypeName = InModelTypeName;
			}
			else if(OutModel && OutModel->Name.HasTypeInfo())
			{
				ModelTypeName = OutModel->Name;
			}
			else
			{
				const FString ModelName = InSrcSchema->Name;
				check(!ModelName.IsEmpty());

				ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
					EWebAPISchemaType::Model,
					NameTransformer(*ModelName),
					*ModelName,
					TEXT("F"));
			}

			if (!InSrcSchema.IsValid())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ModelName"), FText::FromString(*ModelTypeName.ToString(true)));
				MessageLog->LogWarning(
					FText::Format(LOCTEXT("SchemaInvalid",
							"The schema for model \"{ModelName}\" was invalid/null."),
						Args),
					FWebAPISwaggerProvider::LogName);
				return false;
			}

			const TObjectPtr<UWebAPIModel>& Model = OutModel ? OutModel : OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());

			UWebAPIModelBase* ModelBase = Model;
			if (!ConvertModelBase(InSrcSchema, ModelBase))
			{
				return false;
			}

			Model->Name = ModelTypeName;

			if (InSrcSchema->Properties.IsSet() && !InSrcSchema->Properties->IsEmpty())
			{
				for (const TTuple<FString, Json::TJsonReference<OpenAPI::V2::FSchema>>& NamePropertyPair : InSrcSchema->Properties.GetValue())
				{
					const FString& SrcPropertyName = NamePropertyPair.Key;
					FString SrcPropertyDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(*ModelTypeName.ToString(true), SrcPropertyName);
					TSharedPtr<OpenAPI::V2::FSchema> SrcPropertyValue = ResolveReference(NamePropertyPair.Value, SrcPropertyDefinitionName);

					if (SrcPropertyValue)
					{
						TObjectPtr<UWebAPIProperty>& DstProperty = Model->Properties.Add_GetRef(NewObject<UWebAPIProperty>(Model));
						PatchProperty(Model->Name,
							FWebAPINameInfo(NameTransformer(SrcPropertyName), SrcPropertyName, ModelTypeName),
							SrcPropertyValue,
							SrcPropertyDefinitionName,
							DstProperty);
					}
				}
			}

			Model->BindToTypeInfo();

			return true;
		}

		template <>
		inline bool FWebAPISwaggerSchemaConverter::PatchModel<OpenAPI::V2::FParameter>(const TSharedPtr<OpenAPI::V2::FParameter>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel)
		{
			FWebAPITypeNameVariant ModelTypeName;
			if(InModelTypeName.IsValid() && InModelTypeName.HasTypeInfo())
			{
				ModelTypeName = InModelTypeName;
			}
			else if(OutModel && OutModel->Name.HasTypeInfo())
			{
				ModelTypeName = OutModel->Name;
			}
			else
			{
				const FString ModelName = InSrcSchema->Name;
				check(!ModelName.IsEmpty());

				ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
					EWebAPISchemaType::Model,
					NameTransformer(*ModelName),
					*ModelName,
					TEXT("F"));
			}

			if (!InSrcSchema.IsValid())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ModelName"), FText::FromString(*ModelTypeName.ToString(true)));
				MessageLog->LogWarning(
					FText::Format(LOCTEXT("SchemaInvalid",
							"The schema for model \"{ModelName}\" was invalid/null."),
						Args),
					FWebAPISwaggerProvider::LogName);
				return false;
			}

			UWebAPIModel* Model = OutModel ? OutModel : OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());

			UWebAPIModelBase* ModelBase = Model;
			if (!ConvertModelBase<OpenAPI::V2::FParameter, UWebAPIModelBase>(InSrcSchema, ModelBase))
			{
				return false;
			}

			Model->Name = ModelTypeName;

			Model->BindToTypeInfo();

			return true;
		}

		template <typename SchemaType>
		TObjectPtr<UWebAPIModel> FWebAPISwaggerSchemaConverter::ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName)
		{
			check(InSrcSchema.IsValid());

			FWebAPITypeNameVariant ModelTypeName;
			if(InModelTypeName.IsValid() && ModelTypeName.HasTypeInfo())
			{
				ModelTypeName = InModelTypeName;
			}
			else
			{
				const FString ModelName = InSrcSchema->Name;
				check(!ModelName.IsEmpty());

				ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
					EWebAPISchemaType::Model,
					NameTransformer(*ModelName),
					*ModelName,
					TEXT("F"));
			}

			const TObjectPtr<UWebAPIModel>& Model = OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());
			if(PatchModel(InSrcSchema, ModelTypeName, Model))
			{
				return Model;
			}

			return nullptr;
		}
}

#undef LOCTEXT_NAMESPACE
