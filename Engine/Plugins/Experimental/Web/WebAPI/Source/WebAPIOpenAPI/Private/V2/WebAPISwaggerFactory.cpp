// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPISwaggerFactory.h"

#include "V2/WebAPISwaggerProvider.h"
#include "WebAPIOpenAPILog.h"
#include "WebAPIDefinition.h"
#include "Dom/WebAPIOperation.h"
#include "Serialization/JsonSerializer.h"

#define LOCTEXT_NAMESPACE "WebAPISwaggerFactory"

struct FWebAPIOperationResponseKeyFuncs : public TDefaultMapKeyFuncs<uint32, TObjectPtr<UWebAPIOperationResponse>, false>
{
	static FORCEINLINE uint32 GetSetKey(const TObjectPtr<UWebAPIOperationResponse>& Element) { return Element->Code; }
	static FORCEINLINE uint32 GetKeyHash(const uint32& Key) { return Key; }
	static FORCEINLINE bool Matches(const uint32& A, const uint32& B) { return (A == B); }
};

UWebAPISwaggerFactory::UWebAPISwaggerFactory()
	: Provider(MakeShared<FWebAPISwaggerProvider>())
{
}

bool UWebAPISwaggerFactory::CanImportWebAPI(const FString& InFileName, const FString& InFileContents)
{
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(InFileContents), JsonObject))
	{
		return false;
	}

	return JsonObject->HasField(TEXT("swagger"));
}

TFuture<bool> UWebAPISwaggerFactory::ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents)
{
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(InFileContents), JsonObject))
	{
		UE_LOGF(LogWebAPIOpenAPI, Error, "Couldn't deserialize file contents as Json");
		return MakeFulfilledPromise<bool>(false).GetFuture();
	}

	// Write raw spec contents
	UWebAPISwaggerAssetData* ImportData = InDefinition->AddOrGetImportedDataCache<UWebAPISwaggerAssetData>("Swagger");
	ImportData->FileContents = InFileContents;

	// Parse spec
	return Provider->ConvertToWebAPISchema(InDefinition)
	.Next([](EWebAPIConversionResult bInConversionResult)
	{
		return bInConversionResult == EWebAPIConversionResult::Succeeded;
	});
}

#undef LOCTEXT_NAMESPACE

