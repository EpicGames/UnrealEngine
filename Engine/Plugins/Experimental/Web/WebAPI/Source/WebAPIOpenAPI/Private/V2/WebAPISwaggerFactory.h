// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/WebAPIOpenAPIFactoryBase.h"

#include "WebAPISwaggerFactory.generated.h"

class FWebAPISwaggerProvider;

UCLASS()
class WEBAPIOPENAPI_API UWebAPISwaggerAssetData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString FileContents;
};

/** Handles importing for Swagger (OpenAPI V2) files. */
UCLASS()
class WEBAPIOPENAPI_API UWebAPISwaggerFactory
	: public UWebAPIOpenAPIFactoryBase
{
	GENERATED_BODY()

public:
	UWebAPISwaggerFactory();
	
	virtual bool CanImportWebAPI(const FString& InFileName, const FString& InFileContents) override;
	virtual TFuture<bool> ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents) override;

protected:
	TSharedPtr<FWebAPISwaggerProvider> Provider;	
};
