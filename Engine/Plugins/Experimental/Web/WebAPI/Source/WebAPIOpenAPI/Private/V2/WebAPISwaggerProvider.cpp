// Copyright Epic Games, Inc. All Rights Reserved.

#include "V2/WebAPISwaggerProvider.h"

#include "Dom/WebAPITypeRegistry.h"
#include "WebAPIDefinition.h"
#include "WebAPIOpenAPILog.h"
#include "WebAPISwaggerFactory.h"
#include "Dom/WebAPISchema.h"
#include "Serialization/JsonSerializer.h"
#include "V2/WebAPISwaggerConverter.h"
#include "V2/WebAPISwaggerSchema.h"

#define LOCTEXT_NAMESPACE "WebAPISwaggerProvider"

TFuture<EWebAPIConversionResult> FWebAPISwaggerProvider::ConvertToWebAPISchema(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition)
{
	TFuture<EWebAPIConversionResult> FailedWithErrorsResult = MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::FailedWithErrors).GetFuture();
	TFuture<EWebAPIConversionResult> FailedWithWarningsResult = MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::FailedWithWarnings).GetFuture();
	
	if(!InDefinition.IsValid())
	{
		UE_LOGF(LogWebAPIOpenAPI, Error, "The WebAPI Definition was invalid.");
		return FailedWithErrorsResult;
	}

	// Get cached file contents from Definition
	const UWebAPISwaggerAssetData* AssetData = InDefinition->AddOrGetImportedDataCache<UWebAPISwaggerAssetData>(TEXT("Swagger"));

	const TSharedPtr<FWebAPIMessageLog> MessageLog = InDefinition->GetMessageLog();
	check(MessageLog.IsValid());

	// Load Json
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(AssetData->FileContents), JsonObject))
	{
		MessageLog->LogError(FText::FromString(TEXT("Couldn't deserialize file contents as Json.")), FWebAPISwaggerProvider::LogName);
		return FailedWithErrorsResult;
	}

	// Parse Swagger JsonObject
	TSharedPtr<UE::WebAPI::OpenAPI::V2::FSwagger> Root = MakeShared<UE::WebAPI::OpenAPI::V2::FSwagger>();
	if(!Root->FromJson(JsonObject.ToSharedRef()))
	{
		MessageLog->LogError(FText::FromString(TEXT("Failed to load the Swagger schema.")), FWebAPISwaggerProvider::LogName);
		return FailedWithErrorsResult;
	}

	// Convert to WebAPI schema
	UWebAPISchema* WebAPISchema = InDefinition->GetWebAPISchema();
	WebAPISchema->TypeRegistry->Clear();
	const TSharedRef<UE::WebAPI::Swagger::FWebAPISwaggerSchemaConverter, ESPMode::ThreadSafe> Converter = MakeShared<UE::WebAPI::Swagger::FWebAPISwaggerSchemaConverter>(
		Root,
		WebAPISchema,
		MessageLog.ToSharedRef(),
		InDefinition->GetProviderSettings());
	if(Converter->Convert())
	{
		if(!WebAPISchema->TypeRegistry->CheckAllNamed())
		{
			return FailedWithWarningsResult;
		}
		
		return MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::Succeeded).GetFuture();
	}

	MessageLog->LogError(FText::FromString(TEXT("Failed to convert the Swagger schema to the WebAPI schema.")), FWebAPISwaggerProvider::LogName);
	return FailedWithErrorsResult;
}

#undef LOCTEXT_NAMESPACE
