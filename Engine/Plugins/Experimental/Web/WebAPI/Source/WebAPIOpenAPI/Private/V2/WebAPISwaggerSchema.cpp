// Copyright Epic Games, Inc. All Rights Reserved.

#include "V2/WebAPISwaggerSchema.h"

#include "Algo/ForEach.h"

namespace UE::WebAPI::OpenAPI::V2
{
	bool FItems::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("format"), Format);
		bSetAField |= Json::TryGetField(InJson, TEXT("items"), Items);
		bSetAField |= Json::TryGetField(InJson, TEXT("collectionFormat"), CollectionFormat);
		bSetAField |= Json::TryGetField(InJson, TEXT("default"), Default);
		bSetAField |= Json::TryGetField(InJson, TEXT("maximum"), Maximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMaximum"), ExclusiveMaximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("minimum"), Minimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMinimum"), ExclusiveMinimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxLength"), MaxLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("minLength"), MinLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("pattern"), Pattern);
		bSetAField |= Json::TryGetField(InJson, TEXT("minItems"), MinItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxItems"), MaxItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("uniqueItems"), bUniqueItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("enum"), Enum);
		bSetAField |= Json::TryGetField(InJson, TEXT("multipleOf"), MultipleOf);
		return bSetAField;
	}

	bool FExternalDocumentation::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), Url);
		return bSetAField;
	}

	bool FXML::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("namespace"), Namespace);
		bSetAField |= Json::TryGetField(InJson, TEXT("prefix"), Prefix);
		bSetAField |= Json::TryGetField(InJson, TEXT("attribute"), Attribute);
		bSetAField |= Json::TryGetField(InJson, TEXT("wrapped"), Wrapped);
		return bSetAField;
	}

	bool FSchemaBase::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("default"), Default);
		bSetAField |= Json::TryGetField(InJson, TEXT("maximum"), Maximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMaximum"), bExclusiveMaximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("minimum"), Minimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMinimum"), bExclusiveMinimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxLength"), MaxLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("minLength"), MinLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("pattern"), Pattern);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxItems"), MaxItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("minItems"), MinItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("uniqueItems"), bUniqueItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("required"), bRequired);
		bSetAField |= Json::TryGetField(InJson, TEXT("enum"), Enum);
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("items"), Items);
		return bSetAField;
	}

	bool FSchema::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = Super::FromJson(InJson);
		bSetAField |= Json::TryGetField(InJson, TEXT("format"), Format);
		bSetAField |= Json::TryGetField(InJson, TEXT("title"), Title);
		bSetAField |= Json::TryGetField(InJson, TEXT("allOf"), AllOf);
		bSetAField |= Json::TryGetField(InJson, TEXT("properties"), Properties);
		bSetAField |= Json::TryGetField(InJson, TEXT("additionalProperties"), AdditionalProperties);
		bSetAField |= Json::TryGetField(InJson, TEXT("discriminator"), Discriminator);
		bSetAField |= Json::TryGetField(InJson, TEXT("readOnly"), bReadOnly);
		bSetAField |= Json::TryGetField(InJson, TEXT("xml"), XML);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		bSetAField |= Json::TryGetField(InJson, TEXT("example"), Example);
		return bSetAField;
	}

	bool FParameter::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = Super::FromJson(InJson);
		bSetAField |= Json::TryGetField(InJson, TEXT("in"), In);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("format"), Format);
		bSetAField |= Json::TryGetField(InJson, TEXT("schema"), Schema);
		bSetAField |= Json::TryGetField(InJson, TEXT("multipleOf"), MultipleOf);
		return bSetAField;
	}

	bool FHeader::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("format"), Format);
		bSetAField |= Json::TryGetField(InJson, TEXT("items"), Items);
		bSetAField |= Json::TryGetField(InJson, TEXT("collectionFormat"), CollectionFormat);
		bSetAField |= Json::TryGetField(InJson, TEXT("default"), Default);
		bSetAField |= Json::TryGetField(InJson, TEXT("maximum"), Maximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMaximum"), bExclusiveMaximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("minimum"), Minimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMinimum"), bExclusiveMinimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxLength"), MaxLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("minLength"), MinLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("pattern"), Pattern);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxItems"), MaxItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("minItems"), MinItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("uniqueItems"), bUniqueItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("enum"), Enum);
		bSetAField |= Json::TryGetField(InJson, TEXT("multipleOf"), MultipleOf);
		return bSetAField;
	}

	bool FExample::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		const bool bSetAField = false;
		return bSetAField;
	}

	bool FResponse::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("schema"), Schema);
		bSetAField |= Json::TryGetField(InJson, TEXT("headers"), Headers);
		bSetAField |= Json::TryGetField(InJson, TEXT("examples"), Examples);
		return bSetAField;
	}

	bool FScopes::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		const bool bSetAField = false;
		return bSetAField;
	}

	bool FSecurityScheme::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("in"), In);
		bSetAField |= Json::TryGetField(InJson, TEXT("flow"), Flow);
		bSetAField |= Json::TryGetField(InJson, TEXT("authorizationUrl"), AuthorizationUrl);
		bSetAField |= Json::TryGetField(InJson, TEXT("tokenUrl"), TokenUrl);
		bSetAField |= Json::TryGetField(InJson, TEXT("scopes"), Scopes);
		return bSetAField;
	}

	bool FSecurityRequirement::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		const bool bSetAField = false;
		return bSetAField;
	}

	bool FTag::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		return bSetAField;
	}

	bool FPath::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("get"), Get);
		bSetAField |= Json::TryGetField(InJson, TEXT("put"), Put);
		bSetAField |= Json::TryGetField(InJson, TEXT("post"), Post);
		bSetAField |= Json::TryGetField(InJson, TEXT("delete"), Delete);
		bSetAField |= Json::TryGetField(InJson, TEXT("options"), Options);
		bSetAField |= Json::TryGetField(InJson, TEXT("head"), Head);
		bSetAField |= Json::TryGetField(InJson, TEXT("patch"), Patch);
		bSetAField |= Json::TryGetField(InJson, TEXT("parameters"), Parameters);
		return bSetAField;
	}

	bool FSwagger::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;

		bSetAField |= Json::TryGetField(InJson, TEXT("swagger"), Swagger);
		bSetAField |= Json::TryGetField(InJson, TEXT("definitions"), Definitions);
		bSetAField |= Json::TryGetField(InJson, TEXT("info"), Info);
		bSetAField |= Json::TryGetField(InJson, TEXT("host"), Host);
		bSetAField |= Json::TryGetField(InJson, TEXT("basePath"), BasePath);
		bSetAField |= Json::TryGetField(InJson, TEXT("schemes"), Schemes);
		bSetAField |= Json::TryGetField(InJson, TEXT("consumes"), Consumes);
		bSetAField |= Json::TryGetField(InJson, TEXT("produces"), Produces);
		bSetAField |= Json::TryGetField(InJson, TEXT("paths"), Paths);
		bSetAField |= Json::TryGetField(InJson, TEXT("parameters"), Parameters);
		bSetAField |= Json::TryGetField(InJson, TEXT("responses"), Responses);
		bSetAField |= Json::TryGetField(InJson, TEXT("securityDefinitions"), SecurityDefinitions);
		bSetAField |= Json::TryGetField(InJson, TEXT("security"), Security);
		bSetAField |= Json::TryGetField(InJson, TEXT("tags"), Tags);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);

		Algo::ForEachIf(Definitions.Get({}),
		[](const TPair<FString, TSharedPtr<FSchema>>& KVP)
		{
			return KVP.Value->Name.IsEmpty();
		},
		[](const TPair<FString, TSharedPtr<FSchema>>& KVP)
		{
			KVP.Value->Name = KVP.Key;				
		});

		Algo::ForEachIf(Parameters.Get({}),
		[](const TPair<FString, TSharedPtr<FParameter>>& KVP)
		{
			return KVP.Value->Name.IsEmpty();
		},
		[](const TPair<FString, TSharedPtr<FParameter>>& KVP)
		{
			KVP.Value->Name = KVP.Key;				
		});

		Algo::ForEachIf(Responses.Get({}),
		[](const TPair<FString, TSharedPtr<FResponse>>& KVP)
		{
			return !KVP.Value->Name.IsSet();
		},
		[](const TPair<FString, TSharedPtr<FResponse>>& KVP)
		{
			KVP.Value->Name = KVP.Key;		
		});

		Algo::ForEachIf(SecurityDefinitions.Get({}),
		[](const TPair<FString, TSharedPtr<FSecurityScheme>>& KVP)
		{
			return !KVP.Value->Name.IsSet();
		},
		[](const TPair<FString, TSharedPtr<FSecurityScheme>>& KVP)
		{
			KVP.Value->Name = KVP.Key;		
		});

		return bSetAField;
	}

	bool FLicense::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), URL);
		return bSetAField;
	}

	bool FContact::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), URL);
		bSetAField |= Json::TryGetField(InJson, TEXT("email"), Email);
		return bSetAField;
	}

	bool FInfo::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("title"), Title);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("termsOfService"), TermsOfService);
		bSetAField |= Json::TryGetField(InJson, TEXT("contact"), Contact);
		bSetAField |= Json::TryGetField(InJson, TEXT("license"), License);
		bSetAField |= Json::TryGetField(InJson, TEXT("version"), Version);
		return bSetAField;
	}

	bool FOperation::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("tags"), Tags);
		bSetAField |= Json::TryGetField(InJson, TEXT("summary"), Summary);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		bSetAField |= Json::TryGetField(InJson, TEXT("operationId"), OperationId);
		bSetAField |= Json::TryGetField(InJson, TEXT("consumes"), Consumes);
		bSetAField |= Json::TryGetField(InJson, TEXT("produces"), Produces);
		bSetAField |= Json::TryGetField(InJson, TEXT("parameters"), Parameters);
		bSetAField |= Json::TryGetField(InJson, TEXT("responses"), Responses);
		bSetAField |= Json::TryGetField(InJson, TEXT("schemes"), Schemes);
		bSetAField |= Json::TryGetField(InJson, TEXT("security"), Security);
		return bSetAField;
	}

	bool FReference::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("$ref"), $Ref);
		return bSetAField;
	}
}
