// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOpenAPIConverter.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIType.h"
#include "Dom/WebAPITypeRegistry.h"
#include "WebAPIOpenAPIConverter.inl"

#include "IWebAPIEditorModule.h"
#include "WebAPITypes.h"
#include "Algo/ForEach.h"
#include "Dom/WebAPIEnum.h"
#include "Dom/WebAPIOperation.h"
#include "Dom/WebAPIParameter.h"
#include "Dom/WebAPIService.h"
#include "Dom/WebAPITypeRegistry.h"
#include "V3/WebAPIOpenAPIProvider.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPIConverter"

#define SET_OPTIONAL(SrcProperty, DstProperty)			\
if(SrcProperty.IsSet())									\
{														\
	DstProperty = SrcProperty.GetValue();				\
}

#define SET_OPTIONAL_FLAGGED(SrcProperty, DstProperty, DstFlag)	\
if(SrcProperty.IsSet())											\
{																\
	DstFlag = true;												\
	DstProperty = SrcProperty.GetValue();						\
}

namespace UE::WebAPI::OpenAPI
{
	FWebAPIOpenAPISchemaConverter::FWebAPIOpenAPISchemaConverter(
		const TSharedPtr<const UE::WebAPI::OpenAPI::V3::FOpenAPIObject>& InOpenAPI, UWebAPISchema* InWebAPISchema,
		const TSharedRef<FWebAPIMessageLog>& InMessageLog, const FWebAPIProviderSettings& InProviderSettings):
		InputSchema(InOpenAPI)
		, OutputSchema(InWebAPISchema)
		, MessageLog(InMessageLog)
		, ProviderSettings(InProviderSettings)
	{
	}

	bool FWebAPIOpenAPISchemaConverter::Convert()
	{
		if(InputSchema->Info.IsValid())
		{
			OutputSchema->APIName = InputSchema->Info->Title;
			OutputSchema->Version = InputSchema->Info->Version;
		}

		// V3 has multiple servers vs singular in V2
		if(InputSchema->Servers.IsSet()
			&& !InputSchema->Servers->IsEmpty()
			&& InputSchema->Servers.GetValue()[0].IsValid())
		{
			const TSharedPtr<V3::FServerObject> Server = InputSchema->Servers.GetValue()[0];
			
			FString Url = Server->Url;
			FString Scheme;
			FParse::SchemeNameFromURI(*Url, Scheme);

			// If Url isn't complete (relative, etc.), it won't have a scheme.
			if(!Scheme.IsEmpty())
			{
				OutputSchema->URISchemes.Add(Scheme);
			}

			Url = Url.Replace(*(Scheme + TEXT("://")), TEXT(""));					
			Url.Split(TEXT("/"), &OutputSchema->Host, &OutputSchema->BaseUrl);

			// If Url isn't complete, there may not be a valid host
			if(OutputSchema->Host.IsEmpty())
			{
				MessageLog->LogWarning(LOCTEXT("NoHostProvided", "The specification did not contain a host Url, this should be specified manually in the generated project settings."), FWebAPIOpenAPIProvider::LogName);
			}
		}

		// If no Url schemes provided, add https as default
		if(OutputSchema->URISchemes.IsEmpty())
		{
			OutputSchema->URISchemes = { TEXT("https") };
		}
		
		// Top level decl of tags optional, so find from paths
		bool bSuccessfullyConverted = InputSchema->Tags.IsSet()
									? ConvertTags(InputSchema->Tags.GetValue(), OutputSchema.Get())
									: true;

		bSuccessfullyConverted &= InputSchema->Components.IsValid()
								  ? ConvertModels(InputSchema->Components->Schemas, OutputSchema.Get())
								  : true;
		
		bSuccessfullyConverted &= ConvertPaths(InputSchema->Paths, OutputSchema.Get());

		return bSuccessfullyConverted;
	}

	FString FWebAPIOpenAPISchemaConverter::NameTransformer(const FWebAPINameVariant& InString) const
	{
		return ProviderSettings.ToPascalCase(InString);
	}

	TObjectPtr<UWebAPITypeInfo> FWebAPIOpenAPISchemaConverter::ResolveMappedType(const FString& InType)
	{
		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
		
		// OpenAPI type to UE type (Prefix, Name)
		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> TypeMap =
		{
			{TEXT("file"), StaticTypeRegistry->FilePath},
			{TEXT("any"), StaticTypeRegistry->Object},
			{TEXT("object"), StaticTypeRegistry->Object},

			{TEXT("array"), StaticTypeRegistry->String},
			{TEXT("boolean"), StaticTypeRegistry->Boolean},
			{TEXT("byte"), StaticTypeRegistry->Byte},
			{TEXT("integer"), StaticTypeRegistry->Int32},
			{TEXT("int"), StaticTypeRegistry->Int32},
			{TEXT("int32"), StaticTypeRegistry->Int32},
			{TEXT("short"), StaticTypeRegistry->Int16},
			{TEXT("int16"), StaticTypeRegistry->Int16},
			{TEXT("long"), StaticTypeRegistry->Int64},
			{TEXT("int64"), StaticTypeRegistry->Int64},
			{TEXT("float"), StaticTypeRegistry->Float},
			{TEXT("double"), StaticTypeRegistry->Double},

			{TEXT("number"), StaticTypeRegistry->Int32},
			{TEXT("char"), StaticTypeRegistry->Char},
			{TEXT("date"), StaticTypeRegistry->DateTime},
			{TEXT("date-time"), StaticTypeRegistry->DateTime},
			{TEXT("password"), StaticTypeRegistry->String},
			{TEXT("string"), StaticTypeRegistry->String},
			{TEXT("void"), StaticTypeRegistry->Void},
			{TEXT("null"), StaticTypeRegistry->Nullptr}
		};

		if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = TypeMap.Find(InType))
		{
			return *FoundTypeInfo;
		}

		return nullptr;
	}

	TObjectPtr<UWebAPITypeInfo> FWebAPIOpenAPISchemaConverter::ResolveType(
		FString InType,
		FString InFormat,
		FString InDefinitionName,
		const TSharedPtr<OpenAPI::V3::FSchemaObject>& InSchema)
	{
		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
		
		static TMap<EJson, FString> EJson_ValueToTypeName = {
			{EJson::None, TEXT("None")},
			{EJson::Null, TEXT("nullptr")},
			{EJson::String, TEXT("FString")},
			{EJson::Boolean, TEXT("bool")},
			{EJson::Number, TEXT("int32")},
			{EJson::Array, TEXT("TArray")},
			{EJson::Object, TEXT("UObject")},
		};

		static TMap<EJson, TObjectPtr<UWebAPITypeInfo>> EJson_ValueToTypeInfo = {
			{EJson::None, StaticTypeRegistry->Void},
			{EJson::Null, StaticTypeRegistry->Nullptr},
			{EJson::String, StaticTypeRegistry->String},
			{EJson::Boolean, StaticTypeRegistry->Boolean},
			{EJson::Number, StaticTypeRegistry->Int32},
			{EJson::Array, StaticTypeRegistry->String},
			{EJson::Object, StaticTypeRegistry->Object},
		};

		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> JsonTypeValueToTypeInfo = {
			{TEXT("None"), StaticTypeRegistry->Void},
			{TEXT("Null"), StaticTypeRegistry->Nullptr},
			{TEXT("String"), StaticTypeRegistry->String},
			{TEXT("Boolean"), StaticTypeRegistry->Boolean},
			{TEXT("Number"), StaticTypeRegistry->Int32},
			{TEXT("Array"), StaticTypeRegistry->String},
			{TEXT("Object"), StaticTypeRegistry->Object},
		};

		TObjectPtr<UWebAPITypeInfo> Result = nullptr;

		// If a definition name is supplied, try to find it first
		if ((InType == TEXT("Object") || InType == TEXT("Array")) && !InDefinitionName.IsEmpty())
		{
			if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, InDefinitionName))
			{
				Result = *FoundTypeInfo;
			}
		}

		// If not found above
		if (!Result)
		{
			// Try specific types
			if (const TObjectPtr<UWebAPITypeInfo>& FoundMappedTypeInfo = ResolveMappedType(InFormat.IsEmpty() ? InType : InFormat))
			{
				return FoundMappedTypeInfo;
			}
			// Fallback to basic types
			else if (const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = JsonTypeValueToTypeInfo.Find(InType))
			{
				Result = *FoundTypeInfo;
			}

			// If it's not a built-in type
			if (Result != nullptr && !Result->bIsBuiltinType)
			{
				// Duplicate it with the provided definition name 
				if (!InDefinitionName.IsEmpty())
				{
					// Allow prefix to be set later depending on this?
					Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
						EWebAPISchemaType::Model,
						InDefinitionName,
						InDefinitionName,
						Result);
					Result->Prefix = TEXT("F");
					ensure(!Result->Name.IsEmpty());
				}
				else
				{
					checkNoEntry();
					Result = Result->Duplicate(OutputSchema->TypeRegistry);
				}
			}
		}

		return Result;
	}

	TObjectPtr<UWebAPITypeInfo> FWebAPIOpenAPISchemaConverter::GetTypeForContentType(const FString& InContentType)
	{
		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
		
		static TMap<FString, TObjectPtr<UWebAPITypeInfo>> ContentTypeToTypeInfo = {
			{TEXT("application/json"), StaticTypeRegistry->JsonObject},
			{TEXT("application/xml"), StaticTypeRegistry->String},
			{TEXT("text/plain"), StaticTypeRegistry->String}
		};
		
		if(const TObjectPtr<UWebAPITypeInfo>* FoundTypeInfo = ContentTypeToTypeInfo.Find(InContentType.ToLower()))
		{
			return *FoundTypeInfo;
		}

		// For all other cases, use string
		return StaticTypeRegistry->String;
	}

	FWebAPINameVariant FWebAPIOpenAPISchemaConverter::ResolvePropertyName(const TObjectPtr<UWebAPIProperty>& InProperty, const FWebAPITypeNameVariant& InPotentialName, const TOptional<bool>& bInIsArray)
	{
		check(InProperty);

		const bool bIsArray = bInIsArray.Get(InProperty->bIsArray);

		// If it's an array, it may be called the generic "Values", try to find a better name
		if(bIsArray && InProperty->Name == ProviderSettings.GetDefaultArrayPropertyName())
		{
			if(InPotentialName.IsValid())
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(ProviderSettings.Pluralize(InPotentialName.ToString(true)), InProperty->Name.GetJsonName());
				return NameInfo;
			}
			
			if(InProperty->Type.HasTypeInfo() && !InProperty->Type.TypeInfo->bIsBuiltinType)
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(ProviderSettings.Pluralize(InProperty->Type.ToString(true)), InProperty->Name.GetJsonName());
				return NameInfo;
			}
		}
		else if(InProperty->Name == ProviderSettings.GetDefaultPropertyName())
		{
			if(InPotentialName.IsValid())
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(InPotentialName.ToString(true), InProperty->Name.GetJsonName());
				return NameInfo;
			}
			
			if(InProperty->Type.HasTypeInfo() && !InProperty->Type.TypeInfo->bIsBuiltinType)
			{
				FWebAPINameInfo NameInfo = FWebAPINameInfo(InProperty->Type.ToString(true), InProperty->Name.GetJsonName());
				return NameInfo;
			}
		}

		return InProperty->Name;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const FWebAPITypeNameVariant& InPropertyTypeName, UWebAPIProperty* OutProperty)
	{
		OutProperty->Type = InPropertyTypeName;
		check(OutProperty->Type.IsValid());

		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);
		OutProperty->Name = ResolvePropertyName(OutProperty, OutProperty->Type.ToString(true), {});
		OutProperty->bIsRequired = false;
		OutProperty->BindToTypeInfo();

		return true;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertOperationParameter(
		const FWebAPINameVariant& InParameterName,
		const TSharedPtr<OpenAPI::V3::FParameterObject>& InParameter,
		const FString& InDefinitionName,
		UWebAPIOperationParameter* OutParameter)
	{
		// Will get schema or create if it doesn't exist (but will be empty)
		const TSharedPtr<V3::FSchemaObject> ParameterSchema = ResolveReference(InParameter->Schema.Get({}));

		static TMap<FString, EWebAPIParameterStorage> InToStorage = {
			{TEXT("query"), EWebAPIParameterStorage::Query},
			{TEXT("header"), EWebAPIParameterStorage::Header},
			{TEXT("path"), EWebAPIParameterStorage::Path},
			{TEXT("formData"), EWebAPIParameterStorage::Body},
			{TEXT("body"), EWebAPIParameterStorage::Body}
		};

		OutParameter->Storage = InToStorage[InParameter->In];

		if (ParameterSchema.IsValid())
		{
			OutParameter->Type = ResolveType(ParameterSchema, InDefinitionName);
		}
		
		OutParameter->bIsArray = IsArray(InParameter);

		UWebAPIModelBase* ModelBase = Cast<UWebAPIModelBase>(OutParameter);
		if (!ConvertModelBase(InParameter, ModelBase))
		{
			return false;
		}

		OutParameter->Name = FWebAPINameInfo(InParameterName.ToString(true), InParameterName.GetJsonName(), OutParameter->Type);

		// Add struct as it's own model, and reference it as this properties type
		if (OutParameter->Type.ToString(true).IsEmpty())
		{
			const FWebAPITypeNameVariant ModelTypeName = OutParameter->Type;
			ModelTypeName.TypeInfo->SetName(InDefinitionName);
			ModelTypeName.TypeInfo->Prefix = TEXT("F");

			UWebAPIModel* Model = OutputSchema->AddModel<UWebAPIModel>(
				ModelTypeName.HasTypeInfo() ? ModelTypeName.TypeInfo.Get() : nullptr);
			ConvertModel(InParameter->Schema.GetValue().GetShared(), {}, Model);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitModelForParameterOfOperation", "Implicit model created for parameter \"{ParameterName}\" of operation \"{OperationName}\"."),
				TEXT("ParameterName"), FText::FromString(*InParameterName.ToString(true)),
				TEXT("OperationName"), FText::FromString(TEXT("?")));
			MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);
			
			Model->Name.TypeInfo->DebugString += LogMessage.ToString();
			
			Model->Name.TypeInfo->JsonName = InParameterName.GetJsonName();
			Model->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;

			OutParameter->Type = Model->Name;
			OutParameter->Type.TypeInfo->SetModel(Model);
		}

		// Special case for "body" parameters
		if (InParameter->In == TEXT("body"))
		{
			if(OutParameter->Name.ToString(true) == TEXT("Body"))
			{
				OutParameter->Name = InDefinitionName;
			}

			if (OutParameter->bIsArray)
			{
				FString Name = OutParameter->Name.ToString(true);
				Name = ProviderSettings.Singularize(Name);
				Name = ProviderSettings.Pluralize(Name);
				OutParameter->Name = Name;
			}
		}

		OutParameter->Description = InParameter->Description.Get(TEXT(""));
		OutParameter->bIsRequired = InParameter->bRequired.Get(false);
		OutParameter->BindToTypeInfo();

		return true;
	}

	TObjectPtr<UWebAPIParameter> FWebAPIOpenAPISchemaConverter::ConvertParameter(const TSharedPtr<OpenAPI::V3::FParameterObject>& InSrcParameter)
	{
		const FString* ParameterName = nullptr;
		for (const TPair<FString, Json::TJsonReference<V3::FParameterObject>>& ParameterPair : InputSchema->Components->Parameters)
		{
			if(ParameterPair.Value.IsSet() && ParameterPair.Value.GetShared() == InSrcParameter)
			{
				ParameterName = &ParameterPair.Key;
				break;
			}
		}

		check(ParameterName);
		
		if(!ParameterName)
		{
			return nullptr;
		}

		const FString ParameterJsonName = InSrcParameter->Name.IsEmpty() ? *ParameterName : InSrcParameter->Name;

		const FWebAPITypeNameVariant ParameterTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Parameter,
			ProviderSettings.MakeParameterTypeName(NameTransformer(*ParameterName)),
			ParameterJsonName,
			TEXT("F"));
		
		ParameterTypeName.TypeInfo->Suffix = TEXT("Parameter");

		// Don't use ParameterTypeName->Name, it might have a Parameter specific pre/postfix.
		FString SrcParameterDefinitionName = NameTransformer(*ParameterName);

		// Will get schema or create if it doesn't exist (but will be empty)
		const TSharedPtr<OpenAPI::V3::FSchemaObject> SrcParameterSchema = ResolveReference(InSrcParameter->Schema.Get({}), SrcParameterDefinitionName, false);

		const TObjectPtr<UWebAPIParameter> DstParameter = OutputSchema->AddParameter(ParameterTypeName.TypeInfo.Get());

		static TMap<FString, EWebAPIParameterStorage> InToStorage = {
			{TEXT("query"), EWebAPIParameterStorage::Query},
			{TEXT("header"), EWebAPIParameterStorage::Header},
			{TEXT("path"), EWebAPIParameterStorage::Path},
			{TEXT("formData"), EWebAPIParameterStorage::Body},
			{TEXT("body"), EWebAPIParameterStorage::Body}
		};

		DstParameter->Storage = InToStorage[InSrcParameter->In];
		DstParameter->bIsArray = IsArray(InSrcParameter);

		const TObjectPtr<UWebAPIModel> Model = Cast<UWebAPIModel>(DstParameter);
		if (!ConvertModel(InSrcParameter, ParameterTypeName, Model))
		{
			return nullptr;
		}

		// Set the (single) property, will either be a single or array of a model or enum, based on the parameter or it's schema + parameter
		{
			FString PropertyName = DstParameter->bIsArray ? ProviderSettings.GetDefaultArrayPropertyName() : ProviderSettings.GetDefaultPropertyName();
			SrcParameterDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(SrcParameterDefinitionName, PropertyName);
			
			TObjectPtr<UWebAPIProperty>& DstProperty = DstParameter->Property = Model->Properties.Add_GetRef(NewObject<UWebAPIProperty>(Model));
			ConvertProperty(Model->Name,
				PropertyName,
				InSrcParameter,
				SrcParameterDefinitionName,
				DstProperty);

			// Add enum as it's own model, and reference it as this properties type
			if (SrcParameterSchema.IsValid() && SrcParameterSchema->Enum.IsSet() && !SrcParameterSchema->Enum->IsEmpty())
			{
				const FWebAPITypeNameVariant EnumTypeName = DstParameter->Type;
				EnumTypeName.TypeInfo->SetName(ProviderSettings.MakeParameterTypeName(*ParameterName));
				EnumTypeName.TypeInfo->SetNested(ParameterTypeName);

				const UWebAPIEnum* Enum = ConvertEnum(SrcParameterSchema, EnumTypeName);

				const FText LogMessage = FText::FormatNamed(
					LOCTEXT("AddedImplicitEnumForParameter", "Implicit enum created for parameter \"{Name}\"."),
					TEXT("Name"), FText::FromString(**ParameterName));
				MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);

				Enum->Name.TypeInfo->DebugString += LogMessage.ToString();

				DstParameter->Type = Enum->Name;
				DstParameter->Type.TypeInfo->SetModel(Enum);
			}
			// Add struct as it's own model, and reference it as this properties type
			else if (DstParameter->Type.ToString(true).IsEmpty())
			{
				const FWebAPITypeNameVariant ModelTypeName = DstParameter->Name;
				ModelTypeName.TypeInfo->SetName(SrcParameterDefinitionName);
				ModelTypeName.TypeInfo->Prefix = TEXT("F");
				ModelTypeName.TypeInfo->SetNested(ParameterTypeName);

				//const TObjectPtr<UWebAPIModel>& Model = OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.HasTypeInfo() ? ModelTypeName.TypeInfo.Get() : nullptr);
				ConvertModel(InSrcParameter, {}, DstParameter);

				const FText LogMessage = FText::FormatNamed(
					LOCTEXT("AddedImplicitModelForParameter", "Implicit model created for parameter \"{ParameterName}\"."),
					TEXT("ParameterName"), FText::FromString(**ParameterName));
				MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);
				
				DstParameter->Name.TypeInfo->DebugString += LogMessage.ToString();
				
				DstParameter->Name.TypeInfo->JsonName = ParameterTypeName.GetJsonName();
				DstParameter->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
			}
		}

		// Special case for "body" parameters
		if (InSrcParameter->In == TEXT("body"))
		{
			DstParameter->Name = SrcParameterDefinitionName;

			if (DstParameter->bIsArray)
			{
				DstParameter->Name = ProviderSettings.Pluralize(SrcParameterDefinitionName);
			}
		}

		DstParameter->BindToTypeInfo();

		//DstParameter->Name = FWebAPINameInfo(NameTransformer(DstParameter->Name), DstParameter->Name.GetJsonName(), DstParameter->Type);
		DstParameter->Description = InSrcParameter->Description.Get(TEXT(""));
		DstParameter->bIsRequired = InSrcParameter->bRequired.Get(false);
	
		return DstParameter;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertRequest(const FWebAPITypeNameVariant& InOperationName, const TSharedPtr<OpenAPI::V3::FOperationObject>& InOperation, UWebAPIOperationRequest* OutRequest)
	{
		const FWebAPITypeNameVariant RequestTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Model,
			ProviderSettings.MakeRequestTypeName(InOperationName),
			InOperationName.ToString(true),
			IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object);
#if WITH_WEBAPI_DEBUG
		RequestTypeName.TypeInfo->DebugString += TEXT(">ConvertRequest");
#endif
		RequestTypeName.TypeInfo->Prefix = TEXT("F");

		OutRequest->Name = RequestTypeName;

		if(!InOperation->Parameters.IsEmpty())
		{
			for (Json::TJsonReference<OpenAPI::V3::FParameterObject>& SrcParameter : InOperation->Parameters)
			{
				FString SrcParameterDefinitionName;
				// Will get schema or create if it doesn't exist (but will be empty)
				TSharedPtr<OpenAPI::V3::FSchemaObject> SrcParameterSchema = ResolveReference(SrcParameter->Schema.Get({}), SrcParameterDefinitionName);

				FWebAPINameVariant ParameterName = FWebAPINameInfo(NameTransformer(SrcParameter->Name), SrcParameter->Name);
				if(SrcParameterDefinitionName.IsEmpty())
				{
					SrcParameterDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(InOperationName, ParameterName);
				}

				const TObjectPtr<UWebAPIOperationParameter> DstParameter = OutRequest->Parameters.Add_GetRef(NewObject<UWebAPIOperationParameter>(OutRequest));
				ConvertOperationParameter(ParameterName, SrcParameter.GetShared(), SrcParameterDefinitionName,	DstParameter);
				DstParameter->BindToTypeInfo();
			
				// Special case - if the name is "body", set as Body property and not Parameter
				if (SrcParameter->Name.Equals(TEXT("body"), ESearchCase::IgnoreCase) && !SrcParameterDefinitionName.IsEmpty())
				{ 
					// Check for existing definition
					if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, SrcParameterDefinitionName))
					{
						DstParameter->Model = Cast<UWebAPIModel>((*FoundGeneratedType)->GetModel());
						return true;
					}
				}
			}
		}
		else if(TSharedPtr<OpenAPI::V3::FRequestBodyObject> SrcRequestBody = ResolveReference(InOperation->RequestBody))
		{
			if(!SrcRequestBody->Content.IsEmpty())
			{
				TSharedPtr<V3::FMediaTypeObject> MediaType = nullptr;
			
				// Choose json entry or whatever is first (if json not found)
				if(const TSharedPtr<V3::FMediaTypeObject>* JsonMediaType = SrcRequestBody->Content.Find(UE::WebAPI::MimeType::NAME_Json.ToString()))
				{
					MediaType = *JsonMediaType;
				}
				else if(!SrcRequestBody->Content.IsEmpty())
				{
					TArray<FString> Keys;
					SrcRequestBody->Content.GetKeys(Keys);
					MediaType = SrcRequestBody->Content[Keys[0]];
				}

				if(MediaType.IsValid())
				{
					FString MediaTypeDefinitionName;
					TSharedPtr<OpenAPI::V3::FSchemaObject> MediaTypeSchema = ResolveReference(MediaType->Schema, MediaTypeDefinitionName);
					
					const TObjectPtr<UWebAPIOperationParameter> DstParameter = OutRequest->Parameters.Add_GetRef(NewObject<UWebAPIOperationParameter>(OutRequest));
					DstParameter->Type = ResolveType(MediaTypeSchema, MediaTypeDefinitionName);
					check(DstParameter->Type.IsValid());
					
					DstParameter->Name = ProviderSettings.GetDefaultPropertyName();
					DstParameter->Name = ResolvePropertyName(DstParameter, DstParameter->Type.ToString(true));
					check(DstParameter->Name.IsValid());

					DstParameter->Storage = EWebAPIParameterStorage::Body;
					DstParameter->BindToTypeInfo();
				}
			}
		}
		// Operation has no parameters, but if arbitrary json is enabled, make a Value property of type JsonObject
		else if(ProviderSettings.bEnableArbitraryJsonPayloads)
		{
			FWebAPINameVariant ParameterName = TEXT("Value");

			const TObjectPtr<UWebAPIOperationParameter> DstParameter = OutRequest->Parameters.Add_GetRef(NewObject<UWebAPIOperationParameter>(OutRequest));
			DstParameter->Type = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->JsonObject;
			DstParameter->Name = FWebAPINameInfo(ParameterName.ToString(true), ParameterName.GetJsonName(), DstParameter->Type);
			DstParameter->bIsMixin = true; // treat as object, not as field
			DstParameter->Storage = EWebAPIParameterStorage::Body;
			DstParameter->BindToTypeInfo();
		}

		OutRequest->BindToTypeInfo();
		
		return true;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertResponse(const FWebAPITypeNameVariant& InOperationName, uint32 InResponseCode, const TSharedPtr<OpenAPI::V3::FResponseObject>& InResponse, UWebAPIOperationResponse* OutResponse)
	{
		const FWebAPITypeNameVariant ResponseTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
			EWebAPISchemaType::Model,
			ProviderSettings.MakeResponseTypeName(InOperationName, InResponseCode),
			InOperationName.ToString(true),
			IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry()->Object);
#if WITH_WEBAPI_DEBUG
		ResponseTypeName.TypeInfo->DebugString += TEXT(">ConvertResponse");
#endif
		ResponseTypeName.TypeInfo->Prefix = TEXT("F");

		check(ResponseTypeName.HasTypeInfo());

		OutResponse->Name = ResponseTypeName;
		OutResponse->Code = InResponseCode;
		OutResponse->Description = InResponse->Description;
		OutResponse->Message = OutResponse->Description;

		if (InResponse->Content.IsSet())
		{
			for(const TPair<FString, Json::TJsonReference<V3::FMediaTypeObject>>& Item : InResponse->Content.GetValue())
			{
				// Will get schema or create if it doesn't exist (but will be empty)
				FString SrcPropertyDefinitionName;
				TSharedPtr<V3::FSchemaObject> SrcResponseSchema = ResolveReference(Item.Value->Schema, SrcPropertyDefinitionName);

				const FString PropertyName = SrcResponseSchema->Type.Get(GetDefaultJsonTypeForStorage(OutResponse->Storage)) == TEXT("array") ? ProviderSettings.GetDefaultArrayPropertyName() : ProviderSettings.GetDefaultPropertyName();
				const TObjectPtr<UWebAPIProperty>& DstProperty = OutResponse->Properties.Add_GetRef(NewObject<UWebAPIProperty>(OutResponse));
				DstProperty->bIsMixin = true;
				ConvertProperty(OutResponse->Name,
					PropertyName,
					SrcResponseSchema,
					SrcPropertyDefinitionName,
					DstProperty);
			}
		}

		if (InResponse->Headers.IsSet())
		{
			for (const TPair<FString, Json::TJsonReference<OpenAPI::V3::FHeaderObject>>& SrcHeader : InResponse->Headers.GetValue())
			{
				FString Key = SrcHeader.Key;
				Json::TJsonReference<OpenAPI::V3::FHeaderObject> Value = SrcHeader.Value; // @todo: handle, maybe as raw string
			}
		}

		OutResponse->BindToTypeInfo();

		return true;
	}

	TObjectPtr<UWebAPIOperation> FWebAPIOpenAPISchemaConverter::ConvertOperation(const FString& InPath, const FString& InVerb, const TSharedPtr<OpenAPI::V3::FOperationObject>& InSrcOperation, const FWebAPITypeNameVariant& InOperationTypeName)
	{
		FString OperationName = InSrcOperation->OperationId.Get(InSrcOperation->Summary.Get(TEXT("")));
		if(OperationName.IsEmpty())
		{
			// ie. GET /pets/{id} == GetPetsById
			
			TArray<FString> SplitPath;
			InPath.ParseIntoArray(SplitPath, TEXT("/"));

			Algo::ForEach(SplitPath, [&](FString& Str)
			{
				Str = NameTransformer(Str);
			});

			OperationName = NameTransformer(InVerb) + FString::Join(SplitPath, TEXT(""));
			
		}
		
		OperationName = NameTransformer(OperationName);
		OperationName = ProviderSettings.MakeValidMemberName(OperationName, InVerb);

		check(!OperationName.IsEmpty());
		
		FWebAPITypeNameVariant OperationTypeName;
		if(InOperationTypeName.IsValid())
		{
			OperationName = InOperationTypeName.ToString(true);
			OperationTypeName = InOperationTypeName;
		}
		else
		{
			OperationTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Operation,
				NameTransformer(*OperationName),
				*OperationName,
				TEXT("U"));
			OperationTypeName.TypeInfo->SetName(OperationName);

#if WITH_WEBAPI_DEBUG
			OperationTypeName.TypeInfo->DebugString += TEXT(">ConvertOperation");
#endif
		}

		// A spec can have no tags, so ensure there's a default
		const FString FirstTag = ProviderSettings.ToPascalCase(InSrcOperation->Tags.Get({TEXT("Default")})[0]);
		const TObjectPtr<UWebAPIService>* Service = OutputSchema->Services.Find(FirstTag);
		checkf(Service, TEXT("An operation must belong to a service!"));

		const TObjectPtr<UWebAPIOperation> DstOperation = (*Service)->Operations.Add_GetRef(NewObject<UWebAPIOperation>(*Service));
		DstOperation->Service = *Service;
		DstOperation->Verb = InVerb;
		DstOperation->Path = InPath;

		DstOperation->Name = OperationTypeName;
		// Choose first non-empty of: description, summary, operation id
		DstOperation->Description = InSrcOperation->Description.Get(
			InSrcOperation->OperationId.Get(InSrcOperation->Summary.Get(
				InSrcOperation->OperationId.Get(TEXT("")))));
		DstOperation->bIsDeprecated = InSrcOperation->bDeprecated.Get(false);

		ConvertRequest(DstOperation->Name, InSrcOperation, DstOperation->Request);

		if (ensure(!InSrcOperation->Responses.IsEmpty()))
		{
			for (const TPair<FString, Json::TJsonReference<OpenAPI::V3::FResponseObject>>& SrcResponse : InSrcOperation->Responses)
			{
				// If "Default", resolves to 0, and means all other unhandled codes (similar to default in switch statement)
				const uint32 Code = FCString::Atoi(*SrcResponse.Key);
				const TObjectPtr<UWebAPIOperationResponse> DstResponse = DstOperation->Responses.Add_GetRef(NewObject<UWebAPIOperationResponse>(DstOperation));
				ConvertResponse(DstOperation->Name, Code, ResolveReference(SrcResponse.Value), DstResponse);

				// If success response (code 200), had no resolved properties but the operation says it returns something, then add that something as a property
				if(DstResponse->Properties.IsEmpty() && !DstOperation->ResponseContentTypes.IsEmpty() && DstResponse->Code == 200)
				{
					const FString PropertyName = ProviderSettings.GetDefaultPropertyName();
					const TObjectPtr<UWebAPIProperty>& DstProperty = DstResponse->Properties.Add_GetRef(NewObject<UWebAPIProperty>(DstResponse));
					DstProperty->bIsMixin = true;
					ConvertProperty(DstResponse->Name,
						PropertyName,
						GetTypeForContentType(DstOperation->ResponseContentTypes[0]),
						DstProperty);
					DstProperty->Name = PropertyName;
					DstResponse->Storage = EWebAPIResponseStorage::Body;
				}
			}
		}

		check(DstOperation->Name.HasTypeInfo());

		const FName OperationObjectName = ProviderSettings.MakeOperationObjectName(*Service, OperationName);
		DstOperation->Rename(*OperationObjectName.ToString(), DstOperation->GetOuter());
		
		return DstOperation;
	}

	TObjectPtr<UWebAPIService> FWebAPIOpenAPISchemaConverter::ConvertService(const FWebAPINameVariant& InName) const
	{
		const FString TagName = NameTransformer(InName);
		return OutputSchema->GetOrMakeService(TagName);
	}

	TObjectPtr<UWebAPIService> FWebAPIOpenAPISchemaConverter::ConvertService(const TSharedPtr<OpenAPI::V3::FTagObject>& InTag) const
	{
		const TObjectPtr<UWebAPIService> Service = ConvertService(InTag->Name);
		Service->Description = InTag->Description.Get(TEXT(""));
		return Service;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertModels(const TMap<FString, Json::TJsonReference<OpenAPI::V3::FSchemaObject>>& InSchemas,	UWebAPISchema* OutSchema)
	{
		bool bAllConverted = true;
		for (const TTuple<FString, Json::TJsonReference<OpenAPI::V3::FSchemaObject>>& Item : InSchemas)
		{
			const FString& Name = Item.Key;
			const TSharedPtr<V3::FSchemaObject> Schema = ResolveReference(Item.Value);

			if (!Schema.IsValid())
			{
				bAllConverted = false;
				FFormatNamedArguments Args;
				Args.Add(TEXT("ModelName"), FText::FromString(Name));
				MessageLog->LogWarning(
					FText::Format(LOCTEXT("SchemaInvalid", "The schema for model \"{ModelName}\" was invalid/null."), Args),
					FWebAPIOpenAPIProvider::LogName);
				continue;
			}

			FWebAPITypeNameVariant ModelName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(Name),
				Name,
				TEXT("F"));

			const TObjectPtr<UWebAPIModel>& Model = OutSchema->AddModel<UWebAPIModel>(ModelName.TypeInfo.Get());
			ConvertModel(Schema, {}, Model);
		}

		// Set Model property of TypeInfos where applicable
		for (const TObjectPtr<UWebAPIModelBase>& ModelBase : OutputSchema->Models)
		{
			if (const TObjectPtr<UWebAPIModel>& Model = Cast<UWebAPIModel>(ModelBase))
			{
				Model->BindToTypeInfo();
			}
			else if (const TObjectPtr<UWebAPIEnum>& Enum = Cast<UWebAPIEnum>(ModelBase))
			{
				Enum->BindToTypeInfo();
			}
			else if (const TObjectPtr<UWebAPIParameter>& ParameterModel = Cast<UWebAPIParameter>(ModelBase))
			{
				ParameterModel->BindToTypeInfo();
			}
		}

		return bAllConverted;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertParameters(const TArray<Json::TJsonReference<OpenAPI::V3::FParameterObject>>& InParameters, UWebAPISchema* OutSchema)
	{
		bool bAllConverted = true;
		for (const Json::TJsonReference<OpenAPI::V3::FParameterObject>& Item : InParameters)
		{
			bAllConverted &= ConvertParameter(ResolveReference(Item)) != nullptr;
		}

		// Re-bind models to their TypeInfos
		for (const TObjectPtr<UWebAPIModelBase>& Model : OutputSchema->Models)
		{
			Model->BindToTypeInfo();
		}

		return bAllConverted;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertParameters(const TMap<FString, TSharedPtr<OpenAPI::V3::FParameterObject>>& InParameters, UWebAPISchema* OutSchema)
	{
		bool bAllConverted = true;
		for (const TTuple<FString, TSharedPtr<OpenAPI::V3::FParameterObject>>& Item : InParameters)
		{
			const FWebAPITypeNameVariant& Name = Item.Key;
			const TSharedPtr<OpenAPI::V3::FParameterObject>& SrcParameter = Item.Value;

			bAllConverted &= ConvertParameter(SrcParameter) != nullptr;
		}

		// Re-bind models to their TypeInfos
		for (const TObjectPtr<UWebAPIModelBase>& Model : OutputSchema->Models)
		{
			Model->BindToTypeInfo();
		}

		return bAllConverted;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertSecurity(const UE::WebAPI::OpenAPI::V3::FOpenAPIObject& InOpenAPI, UWebAPISchema* OutSchema)
	{
		if (!InOpenAPI.Components->SecuritySchemes.IsEmpty())
		{
			for (const TPair<FString, Json::TJsonReference<OpenAPI::V3::FSecuritySchemeObject>>& Item : InOpenAPI.Components->SecuritySchemes)
			{
				const FString& Name = Item.Key;
				const TSharedPtr<OpenAPI::V3::FSecuritySchemeObject>& SecurityScheme = ResolveReference(Item.Value);
				if(SecurityScheme->Type == TEXT("http") && SecurityScheme->Scheme == TEXT("basic"))
				{
					// type = basic
				}
				else if(SecurityScheme->Type == TEXT("http") && SecurityScheme->Scheme == TEXT("bearer"))
				{
					// type = apiKey
					// name = authorization
					// storage = header
				}
				else if(SecurityScheme->Type == TEXT("oauth2"))
				{
					// @todo: handle oauth2 conversion
					FString FlowName = SecurityScheme->Flows.IsValid() ? TEXT("") : TEXT("");
					V3::FOAuthFlowsObject Flow = (SecurityScheme->Flows.Get())[0];

					if(FlowName == TEXT("clientCredentials"))
					{
						// flow = application
					}
					else if(FlowName == TEXT("authorizationCode"))
					{
						// flow = accessCode
					}
					else
					{
						// flow = FlowName
					}
				}

				// @todo: WebAPI security primitive
				// TObjectPtr<UWebAPIModel> Model = OutSchema->Client->Models.Add_GetRef(NewObject<UWebAPIModel>(OutSchema));
				// ConvertModel(Name, Schema.Get(), Model);
			}

			return true;
		}

		return false;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertTags(const TArray<TSharedPtr<OpenAPI::V3::FTagObject>>& InTags, UWebAPISchema* OutSchema) const
	{
		for (const TSharedPtr<OpenAPI::V3::FTagObject>& Tag : InTags)
		{
			ConvertService(Tag);
		}

		return OutSchema->Services.Num() > 0;
	}

	bool FWebAPIOpenAPISchemaConverter::ConvertPaths(const TMap<FString, TSharedPtr<OpenAPI::V3::FPathItemObject>>& InPaths, UWebAPISchema* OutSchema)
	{
		for (const TTuple<FString, TSharedPtr<OpenAPI::V3::FPathItemObject>>& Item : InPaths)
		{
			const FString& Url = Item.Key;
			const TSharedPtr<OpenAPI::V3::FPathItemObject>& Path = Item.Value;

			ConvertParameters(Path->Parameters, OutputSchema.Get());
			
			TMap<FString, TSharedPtr<OpenAPI::V3::FOperationObject>> SrcVerbs;
			if (Path->Get.IsValid())
			{
				SrcVerbs.Add(TEXT("Get"), Path->Get);
			}
			if (Path->Put.IsValid())
			{
				SrcVerbs.Add(TEXT("Put"), Path->Put);
			}
			if (Path->Post.IsValid())
			{
				SrcVerbs.Add(TEXT("Post"), Path->Post);
			}
			if (Path->Delete.IsValid())
			{
				SrcVerbs.Add(TEXT("Delete"), Path->Delete);
			}
			if (Path->Options.IsValid())
			{
				SrcVerbs.Add(TEXT("Options"), Path->Options);
			}
			if (Path->Head.IsValid())
			{
				SrcVerbs.Add(TEXT("Head"), Path->Head);
			}
			if (Path->Patch.IsValid())
			{
				SrcVerbs.Add(TEXT("Patch"), Path->Patch);
			}

			// Each path can have multiple, ie. Get, Put, Delete
			for (const TPair<FString, TSharedPtr<OpenAPI::V3::FOperationObject>>& VerbOperationPair : SrcVerbs)
			{
				FString Verb = VerbOperationPair.Key;
				if (VerbOperationPair.Value == nullptr || !VerbOperationPair.Value.IsValid())
				{
					continue;
				}

				TSharedPtr<OpenAPI::V3::FOperationObject> SrcOperation = VerbOperationPair.Value;

				TArray<FString> Tags;
				if (SrcOperation.IsValid() && SrcOperation->Tags.IsSet())
				{
					Tags = SrcOperation->Tags.GetValue();
				}
				else
				{
					Tags = { TEXT("Default") };
				}

				for (FString& Tag : Tags)
				{
					// Do first to retain original case
					TObjectPtr<UWebAPIService> Service = ConvertService(Tag);
					Tag = NameTransformer(Tag);

					FString OperationName = SrcOperation->OperationId.Get(Verb + Tag);
					OperationName = NameTransformer(OperationName);
					OperationName = ProviderSettings.MakeValidMemberName(OperationName, Verb);

					FString OperationNamePrefix = TEXT("");
					int32 OperationNameSuffix = 0;

					// Check if this already exists - each operation is unique so the name needs to be different - prepend verb
					while(OutSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Operation,
																	 OperationNamePrefix + OperationName + (OperationNameSuffix > 0 ? FString::FormatAsNumber(OperationNameSuffix) : TEXT(""))) != nullptr)
					{
						if(!OperationName.StartsWith(Verb))
						{
							OperationNamePrefix = Verb;
						}
						OperationNameSuffix += 1; 
					}

					OperationName = OperationNamePrefix + OperationName + (OperationNameSuffix > 0 ? FString::FormatAsNumber(OperationNameSuffix) : TEXT(""));
		
					TObjectPtr<UWebAPITypeInfo> OperationTypeInfo = OutSchema->TypeRegistry->
						GetOrMakeGeneratedType(
											   EWebAPISchemaType::Operation,
											   OperationName,
											   OperationName,
											   TEXT("U"));

					const TObjectPtr<UWebAPIOperation> Operation = ConvertOperation(Url, Verb, SrcOperation, OperationTypeInfo);
					Operation->Service = Service;
					Operation->Verb = Verb;
					Operation->Path = Url;

					FStringFormatNamedArguments FormatArgs;
					FormatArgs.Add(TEXT("ClassName"), UWebAPIOperation::StaticClass()->GetName());
					FormatArgs.Add(TEXT("ServiceName"), Service->Name.ToString(true));
					FormatArgs.Add(TEXT("OperationName"), OperationName);

					const FName OperationObjectName = ProviderSettings.MakeOperationObjectName(Service, OperationName);
					Operation->Rename(*OperationObjectName.ToString(), Operation->GetOuter());

					Operation->BindToTypeInfo();
				}
			}
		}
		return true;
	}
};

#undef SET_OPTIONAL_FLAGGED
#undef SET_OPTIONAL

#undef LOCTEXT_NAMESPACE
