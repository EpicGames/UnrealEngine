// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "V3/WebAPIOpenAPISchema.h" // IWYU pragma: keep
#include "WebAPIProviderSettings.h"

class FWebAPIMessageLog;
class UWebAPIEnum;
class UWebAPIModel;
class UWebAPIOperation;
class UWebAPIOperationParameter;
class UWebAPIOperationRequest;
class UWebAPIOperationResponse;
class UWebAPIParameter;
class UWebAPIProperty;
class UWebAPISchema;
class UWebAPITypeInfo;
namespace UE::Json { template <class ObjectType> class TJsonReference; }
template <typename OptionalType> struct TOptional;

namespace UE::WebAPI::OpenAPI
{
	class FWebAPIOpenAPISchemaConverter
	{
	public:
		FWebAPIOpenAPISchemaConverter(
			const TSharedPtr<const UE::WebAPI::OpenAPI::V3::FOpenAPIObject>& InOpenAPI,
			UWebAPISchema* InWebAPISchema,
			const TSharedRef<FWebAPIMessageLog>& InMessageLog,
			const FWebAPIProviderSettings& InProviderSettings);

		bool Convert();

	public:
		FString NameTransformer(const FWebAPINameVariant& InString) const;

		TObjectPtr<UWebAPITypeInfo> ResolveMappedType(const FString& InType);

		TObjectPtr<UWebAPITypeInfo> ResolveType(FString InType, FString InFormat, FString InDefinitionName, const TSharedPtr<OpenAPI::V3::FSchemaObject>& InSchema = nullptr);

		template <typename SchemaType>
		TObjectPtr<UWebAPITypeInfo> ResolveType(const TSharedPtr<SchemaType>& InSchema,	const FString& InDefinitionName = {}, FString InJsonType = {});

		TObjectPtr<UWebAPITypeInfo> GetTypeForContentType(const FString& InContentType);

		template <typename StorageType>
		FString GetDefaultJsonTypeForStorage(const StorageType& InStorage);

		template <typename ObjectType>
		TSharedPtr<ObjectType> ResolveReference(const FString& InDefinitionName);

		/** Only resolves if necessary. */
		template <typename ObjectType>
		TSharedPtr<ObjectType> ResolveReference(const UE::Json::TJsonReference<ObjectType>& InJsonReference, FString& OutDefinitionName, bool bInCheck = true);

		/** Only resolves if necessary. */
		template <typename ObjectType>
		TSharedPtr<ObjectType> ResolveReference(const UE::Json::TJsonReference<ObjectType>& InJsonReference, bool bInCheck = true);

		FWebAPINameVariant ResolvePropertyName(const TObjectPtr<UWebAPIProperty>& InProperty, const FWebAPITypeNameVariant& InPotentialName, const TOptional<bool>& bInIsArray = {});

		template <typename SchemaType>
		bool IsArray(const TSharedPtr<SchemaType>& InSchema);

		template <typename SchemaType, typename ModelType>
		static bool ConvertModelBase(const TSharedPtr<SchemaType>& InSchema, ModelType* OutModel);

		template <typename SchemaType>
		TObjectPtr<UWebAPIEnum> ConvertEnum(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InEnumTypeName = {}) const;

		/** Modifies an existing Property using the supplied source object. */
		template <typename SchemaType>
		bool ConvertProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const TSharedPtr<SchemaType>& InSchema, const FString& InDefinitionName, UWebAPIProperty* OutProperty);

		/** Modifies an existing Property using the supplied TypeInfo. */
		bool ConvertProperty(const FWebAPITypeNameVariant& InModelName, const FWebAPINameVariant& InPropertyName, const FWebAPITypeNameVariant& InPropertyTypeName, UWebAPIProperty* OutProperty);
		
		/** Creates a new Property from the supplied source object. */
		template <typename SchemaType>
		TObjectPtr<UWebAPIProperty> ConvertProperty(const TSharedPtr<SchemaType>& InSrcSchema, const TObjectPtr<UWebAPIModel>& InModel, const FWebAPINameVariant& InPropertyName = {}, const FString& InDefinitionName = {});

		/** Modifies an existing Model using the supplied source object. */
		template <typename SchemaType>
		bool ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel);
		
		/** Creates a new Model from the supplied source object. */
		template <typename SchemaType>
		TObjectPtr<UWebAPIModel> ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName = {});

		bool ConvertOperationParameter(const FWebAPINameVariant& InParameterName, const TSharedPtr<OpenAPI::V3::FParameterObject>& InParameter, const FString& InDefinitionName, UWebAPIOperationParameter* OutParameter);

		TObjectPtr<UWebAPIParameter> ConvertParameter(const TSharedPtr<OpenAPI::V3::FParameterObject>& InSrcParameter);
		
		bool ConvertRequest(const FWebAPITypeNameVariant& InOperationName, const TSharedPtr<OpenAPI::V3::FOperationObject>& InOperation, UWebAPIOperationRequest* OutRequest);

		bool ConvertResponse(const FWebAPITypeNameVariant& InOperationName, uint32 InResponseCode, const TSharedPtr<OpenAPI::V3::FResponseObject>& InResponse, UWebAPIOperationResponse* OutResponse);

		TObjectPtr<UWebAPIOperation> ConvertOperation(const FString& InPath, const FString& InVerb, const TSharedPtr<OpenAPI::V3::FOperationObject>& InSrcOperation, const FWebAPITypeNameVariant& InOperationTypeName = {});

		TObjectPtr<UWebAPIService> ConvertService(const FWebAPINameVariant& InName) const;
		TObjectPtr<UWebAPIService> ConvertService(const TSharedPtr<OpenAPI::V3::FTagObject>& InTag) const;

		bool ConvertModels(const TMap<FString, Json::TJsonReference<OpenAPI::V3::FSchemaObject>>& InSchemas, UWebAPISchema* OutSchema);

		bool ConvertParameters(const TArray<Json::TJsonReference<OpenAPI::V3::FParameterObject>>& InParameters, UWebAPISchema* OutSchema);

		bool ConvertParameters(const TMap<FString, TSharedPtr<OpenAPI::V3::FParameterObject>>& InParameters, UWebAPISchema* OutSchema);

		bool ConvertSecurity(const UE::WebAPI::OpenAPI::V3::FOpenAPIObject& InOpenAPI, UWebAPISchema* OutSchema);

		bool ConvertTags(const TArray<TSharedPtr<OpenAPI::V3::FTagObject>>& InTags, UWebAPISchema* OutSchema) const;

		bool ConvertPaths(const TMap<FString, TSharedPtr<OpenAPI::V3::FPathItemObject>>& InPaths, UWebAPISchema* OutSchema);

	private:
		TSharedPtr<const UE::WebAPI::OpenAPI::V3::FOpenAPIObject> InputSchema;
		TStrongObjectPtr<UWebAPISchema> OutputSchema;
		TSharedPtr<FWebAPIMessageLog> MessageLog;
		FWebAPIProviderSettings ProviderSettings;
	};

	template <>
	TSharedPtr<OpenAPI::V3::FSchemaObject> FWebAPIOpenAPISchemaConverter::ResolveReference(const FString& InDefinitionName);

	template <>
	TSharedPtr<OpenAPI::V3::FParameterObject> FWebAPIOpenAPISchemaConverter::ResolveReference(const FString& InDefinitionName);

	template <>
	TSharedPtr<OpenAPI::V3::FResponseObject> FWebAPIOpenAPISchemaConverter::ResolveReference(const FString& InDefinitionName);

	template <>
	TSharedPtr<OpenAPI::V3::FSecuritySchemeObject> FWebAPIOpenAPISchemaConverter::ResolveReference(const FString& InDefinitionName);

	template <>
	TSharedPtr<OpenAPI::V3::FRequestBodyObject> FWebAPIOpenAPISchemaConverter::ResolveReference(const FString& InDefinitionName);

	/** Modifies an existing Model using the supplied source object. */
	template <>
	bool FWebAPIOpenAPISchemaConverter::ConvertModel(const TSharedPtr<OpenAPI::V3::FParameterObject>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel);
};
