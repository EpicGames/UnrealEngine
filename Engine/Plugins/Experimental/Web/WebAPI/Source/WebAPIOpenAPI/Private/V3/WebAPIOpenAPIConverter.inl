// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebAPIOpenAPIConverter.h"

#include "IWebAPIEditorModule.h"
#include "V3/WebAPIOpenAPIProvider.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPIConverter"

namespace UE::WebAPI::OpenAPI
{
	template <>
	inline FString FWebAPIOpenAPISchemaConverter::GetDefaultJsonTypeForStorage<EWebAPIParameterStorage>(const EWebAPIParameterStorage& InStorage)
	{
		static TMap<EWebAPIParameterStorage, FString> StorageToJsonType = {
			{ EWebAPIParameterStorage::Body, TEXT("object") },
			{ EWebAPIParameterStorage::Cookie, TEXT("string") },
			{ EWebAPIParameterStorage::Header, TEXT("string") },
			{ EWebAPIParameterStorage::Path, TEXT("string") },
			{ EWebAPIParameterStorage::Query, TEXT("string") },
		};

		return StorageToJsonType[InStorage];
	}

	template <>
	inline FString FWebAPIOpenAPISchemaConverter::GetDefaultJsonTypeForStorage<EWebAPIResponseStorage>(const EWebAPIResponseStorage& InStorage)
	{
		static TMap<EWebAPIResponseStorage, FString> StorageToJsonType = {
			{ EWebAPIResponseStorage::Body, TEXT("object") },
			{ EWebAPIResponseStorage::Header, TEXT("string") },
		};

		return StorageToJsonType[InStorage];
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPITypeInfo> FWebAPIOpenAPISchemaConverter::ResolveType(const TSharedPtr<SchemaType>& InSchema, const FString& InDefinitionName, FString InJsonType)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();

		FString DefinitionName = InDefinitionName;
		TObjectPtr<UWebAPITypeInfo> Result = nullptr;
		
		TSharedPtr<UE::WebAPI::OpenAPI::V3::FSchemaObject> ItemSchema = nullptr;
		if(InSchema.IsValid() && InSchema->Items.IsSet())
		{
			ItemSchema = ResolveReference(InSchema->Items.GetValue(), DefinitionName);
			if(!InSchema->Items->GetPath().IsEmpty())
			{
				DefinitionName = InSchema->Items->GetLastPathSegment();						
			}
			return ResolveType(ItemSchema, DefinitionName);
		}

		if(InSchema.IsValid())
		{
			Result = ResolveType(InSchema->Type.Get(InJsonType.IsEmpty() ? TEXT("object") : InJsonType), InSchema->Format.Get(TEXT("")), DefinitionName);
		}
		else
		{
			Result = ResolveType(InJsonType, TEXT(""), DefinitionName);
			if(!Result)
			{
				Result = StaticTypeRegistry->Object;
			}
		}

		if (InSchema.IsValid() && InSchema->Enum.IsSet() && !InSchema->Enum.GetValue().IsEmpty())
		{
			if(const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, NameTransformer(DefinitionName)))
			{
				Result = *FoundGeneratedType;						
			}
			else
			{
				Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
					EWebAPISchemaType::Model,
					NameTransformer(DefinitionName),
					DefinitionName,
					StaticTypeRegistry->Enum);
			}
		}
		else if (!Result->IsEnum() && (Result == StaticTypeRegistry->Object || Result->ToString(true).IsEmpty()))
		{
			if (!DefinitionName.IsEmpty())
			{
				if (const TObjectPtr<UWebAPITypeInfo>* FoundBuiltinType = StaticTypeRegistry->FindBuiltinType(DefinitionName))
				{
					Result = *FoundBuiltinType;
				}
				else if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedModelType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Model, DefinitionName))
				{
					Result = *FoundGeneratedModelType;
				}
				else if (const TObjectPtr<UWebAPITypeInfo>* FoundGeneratedParameterType = OutputSchema->TypeRegistry->FindGeneratedType(EWebAPISchemaType::Parameter, DefinitionName))
				{
					Result = *FoundGeneratedParameterType;
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("DefinitionName"), FText::FromString(DefinitionName));
					MessageLog->LogInfo(FText::Format(LOCTEXT("CannotResolveType", "ResolveType (object) failed to find a matching type for definition \"{DefinitionName}\", creating a new one."), Args), FWebAPIOpenAPIProvider::LogName);

					Result = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
						EWebAPISchemaType::Model,
						NameTransformer(DefinitionName),
						{},
						TEXT("F"));
					Result->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;
				}
			}
		}

		check(Result);
		
		return Result;
	}

	template <>
	inline TSharedPtr<OpenAPI::V3::FSchemaObject> FWebAPIOpenAPISchemaConverter::ResolveReference<OpenAPI::V3::FSchemaObject>(const FString& InDefinitionName)
	{
		if(!InputSchema->Components.IsValid())
		{
			return nullptr;
		}
		
		if (const Json::TJsonReference<OpenAPI::V3::FSchemaObject>* FoundDefinition = InputSchema->Components->Schemas.Find(InDefinitionName))
		{
			if(!FoundDefinition->IsSet())
			{
				return ResolveReference<OpenAPI::V3::FSchemaObject>(FoundDefinition->GetLastPathSegment());
			}
			return FoundDefinition->GetShared();
		}

		return nullptr;
	}

	template <>
	inline TSharedPtr<OpenAPI::V3::FParameterObject> FWebAPIOpenAPISchemaConverter::ResolveReference<OpenAPI::V3::FParameterObject>(const FString& InDefinitionName)
	{
		if(!InputSchema->Components.IsValid())
		{
			return nullptr;
		}
		
		if (const Json::TJsonReference<OpenAPI::V3::FParameterObject>* FoundDefinition = InputSchema->Components->Parameters.Find(InDefinitionName))
		{
			if(!FoundDefinition->IsSet())
			{
				return ResolveReference<OpenAPI::V3::FParameterObject>(FoundDefinition->GetLastPathSegment());
			}
			return FoundDefinition->GetShared();
		}

		return nullptr;
	}

	template <>
	inline TSharedPtr<OpenAPI::V3::FResponseObject> FWebAPIOpenAPISchemaConverter::ResolveReference<V3::FResponseObject>(const FString& InDefinitionName)
	{
		if(!InputSchema->Components.IsValid())
		{
			return nullptr;
		}
		
		if (const Json::TJsonReference<OpenAPI::V3::FResponseObject>* FoundDefinition = InputSchema->Components->Responses.Find(InDefinitionName))
		{
			if(!FoundDefinition->IsSet())
			{
				return ResolveReference<OpenAPI::V3::FResponseObject>(FoundDefinition->GetLastPathSegment());
			}
			return FoundDefinition->GetShared();
		}

		return nullptr;
	}

	template <>
	inline TSharedPtr<OpenAPI::V3::FSecuritySchemeObject> FWebAPIOpenAPISchemaConverter::ResolveReference<V3::FSecuritySchemeObject>(const FString& InDefinitionName)
	{
		if(!InputSchema->Components.IsValid())
		{
			return nullptr;
		}
		
		if (const Json::TJsonReference<OpenAPI::V3::FSecuritySchemeObject>* FoundDefinition = InputSchema->Components->SecuritySchemes.Find(InDefinitionName))
		{
			if(!FoundDefinition->IsSet())
			{
				return ResolveReference<OpenAPI::V3::FSecuritySchemeObject>(FoundDefinition->GetLastPathSegment());
			}
			return FoundDefinition->GetShared();
		}

		return nullptr;
	}

	template <>
	inline TSharedPtr<OpenAPI::V3::FRequestBodyObject> FWebAPIOpenAPISchemaConverter::ResolveReference<V3::FRequestBodyObject>(const FString& InDefinitionName)
	{
		if (const Json::TJsonReference<OpenAPI::V3::FRequestBodyObject>* FoundDefinition = InputSchema->Components->RequestBodies.Find(InDefinitionName))
		{
			if(!FoundDefinition->IsSet())
			{
				return ResolveReference<OpenAPI::V3::FRequestBodyObject>(FoundDefinition->GetLastPathSegment());
			}
			return FoundDefinition->GetShared();
		}

		return nullptr;
	}

	template <typename ObjectType>
	TSharedPtr<ObjectType> FWebAPIOpenAPISchemaConverter::ResolveReference(const Json::TJsonReference<ObjectType>& InJsonReference, FString& OutDefinitionName, bool bInCheck)
	{
		if (InJsonReference.IsSet())
		{
			return InJsonReference.GetShared();
		}

		if(!InJsonReference.IsValid())
		{
			return nullptr;
		}
		
		FString DefinitionName = InJsonReference.GetLastPathSegment();
		if (TSharedPtr<ObjectType> FoundDefinition = ResolveReference<ObjectType>(DefinitionName))
		{
			OutDefinitionName = DefinitionName; // Only set if found 
			return FoundDefinition;
		}

		if(bInCheck)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ReferencePath"), FText::FromString(InJsonReference.GetPath()));
			MessageLog->LogWarning(FText::Format(LOCTEXT("CannotResolveJsonReference", "Couldn't resolve JsonReference \"{ReferencePath}\""), Args),	FWebAPIOpenAPIProvider::LogName);
		}

		return nullptr;
	}

	template <typename ObjectType>
	TSharedPtr<ObjectType> FWebAPIOpenAPISchemaConverter::ResolveReference(const UE::Json::TJsonReference<ObjectType>& InJsonReference, bool bInCheck)
	{
		FString DefinitionName;
		return ResolveReference(InJsonReference, DefinitionName, bInCheck);
	}

	template <>
	inline bool FWebAPIOpenAPISchemaConverter::IsArray(const TSharedPtr<UE::WebAPI::OpenAPI::V3::FSchemaObject>& InSchema)
	{
		return InSchema->Type.Get(TEXT(""))	== TEXT("array");
	}

	template <>
	inline bool FWebAPIOpenAPISchemaConverter::IsArray(const TSharedPtr<UE::WebAPI::OpenAPI::V3::FParameterObject>& InSchema)
	{
		FString FoundDefinitionName;
		return InSchema->Schema.IsSet()
			? IsArray(ResolveReference(InSchema->Schema.GetValue(), FoundDefinitionName))
			: false;
	}

	template <typename SchemaType, typename ModelType>
	bool FWebAPIOpenAPISchemaConverter::ConvertModelBase(const TSharedPtr<SchemaType>& InSchema,
														ModelType* OutModel)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		return true;
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPIEnum> FWebAPIOpenAPISchemaConverter::ConvertEnum(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InEnumTypeName) const
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		FWebAPITypeNameVariant EnumTypeName;
		if(InEnumTypeName.IsValid())
		{
			EnumTypeName = InEnumTypeName;
		}
		else
		{
			const FString EnumName = InSrcSchema->Name;
			check(!EnumName.IsEmpty());

			const TObjectPtr<UWebAPIStaticTypeRegistry> StaticTypeRegistry = IWebAPIEditorModuleInterface::Get().GetStaticTypeRegistry();
			EnumTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(EnumName),
				EnumName,
				StaticTypeRegistry->Enum);
		}

		const TObjectPtr<UWebAPIEnum>& DstEnum = OutputSchema->AddEnum(EnumTypeName.TypeInfo.Get());
		DstEnum->Name = EnumTypeName;
		DstEnum->Description = InSrcSchema->Description.Get(TEXT(""));

		UWebAPIModelBase* ModelBase = Cast<UWebAPIModelBase>(DstEnum);
		if (!ConvertModelBase(InSrcSchema, ModelBase))
		{
			return nullptr;
		}

		for (const FString& SrcEnumValue : InSrcSchema->Enum.GetValue())
		{
			const TObjectPtr<UWebAPIEnumValue> DstEnumValue = DstEnum->AddValue();
			DstEnumValue->Name.NameInfo.Name = NameTransformer(SrcEnumValue);
			DstEnumValue->Name.NameInfo.JsonName = SrcEnumValue;
		}

		DstEnum->BindToTypeInfo();
		
		return DstEnum;
	}

	template <>
	inline bool FWebAPIOpenAPISchemaConverter::ConvertProperty<OpenAPI::V3::FParameterObject>(
		const FWebAPITypeNameVariant& InModelName,
		const FWebAPINameVariant& InPropertyName,
		const TSharedPtr<OpenAPI::V3::FParameterObject>& InParameter,
		const FString& InDefinitionName,
		UWebAPIProperty* OutProperty)
	{
		const FString DefinitionName = !InDefinitionName.IsEmpty()
										? InDefinitionName
										: ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName);

		UWebAPIModelBase* ModelBase = OutProperty;
		if (!ConvertModelBase(InParameter, ModelBase))
		{
			return false;
		}

		OutProperty->bIsArray = IsArray(InParameter);

		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);				
		OutProperty->Name = ResolvePropertyName(OutProperty, InModelName, {});
		OutProperty->bIsRequired = InParameter->bRequired.Get(false);
		OutProperty->BindToTypeInfo();

		if(InParameter->Schema.IsSet())
		{
			const TSharedPtr<V3::FSchemaObject> Schema = ResolveReference(InParameter->Schema.GetValue());
			OutProperty->Type = ResolveType(Schema);
			
			// Add enum as it's own model, and reference it as this properties type
			if (InParameter->Schema.GetValue()->Enum.IsSet() && !InParameter->Schema.GetValue()->Enum->IsEmpty())
			{
				const FWebAPITypeNameVariant EnumTypeName = OutProperty->Type;
				EnumTypeName.TypeInfo->SetNested(InModelName);
				FString EnumName = OutProperty->Name.ToString(true);

				// Only make nested name if the model and property name aren't the same, otherwise you get "NameName"!
				if(EnumName != InModelName)
				{
					EnumName = ProviderSettings.MakeNestedPropertyTypeName(InModelName, OutProperty->Name.ToString(true));
				}
				EnumTypeName.TypeInfo->SetName(EnumName);

				const UWebAPIEnum* Enum = ConvertEnum(Schema, EnumTypeName);

				const FText LogMessage = FText::FormatNamed(
					LOCTEXT("AddedImplicitEnumForPropertyOfModel", "Implicit enum created for property \"{PropertyName}\" of model \"{ModelName}\"."),
					TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
					TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
				MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);
			
				Enum->Name.TypeInfo->DebugString += LogMessage.ToString();
				Enum->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();

				OutProperty->Type = Enum->Name;
				OutProperty->Type.TypeInfo->SetModel(Enum);
			}
		}

		return true;
	}

	template <typename SchemaType>
	bool FWebAPIOpenAPISchemaConverter::ConvertProperty(const FWebAPITypeNameVariant& InModelName,
														const FWebAPINameVariant& InPropertyName,
														const TSharedPtr<SchemaType>& InSchema,
														const FString& InDefinitionName,
														UWebAPIProperty* OutProperty)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		FString DefinitionName = !InDefinitionName.IsEmpty()
								? InDefinitionName
								: ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName);

		TSharedPtr<UE::WebAPI::OpenAPI::V3::FSchemaObject> ItemSchema = InSchema;
		if(InSchema->Items.IsSet())
		{
			if(!InSchema->Items->GetPath().IsEmpty())
			{
				ItemSchema = ResolveReference(InSchema->Items.GetValue(), DefinitionName);
				DefinitionName = InSchema->Items->GetLastPathSegment();						
			}
		}
		else
		{
			ItemSchema = InSchema;
		}

		OutProperty->Type = ResolveType<SchemaType>(ItemSchema, DefinitionName);
		check(OutProperty->Type.IsValid());

		UWebAPIModelBase* ModelBase = OutProperty;
		if (!ConvertModelBase(ItemSchema, ModelBase))
		{
			return false;
		}

		OutProperty->bIsArray = IsArray(InSchema);
		
		OutProperty->Name = FWebAPINameInfo(NameTransformer(InPropertyName.ToString()), InPropertyName.GetJsonName(), OutProperty->Type);
		OutProperty->Name = ResolvePropertyName(OutProperty, OutProperty->Type.ToString(true), {});
		OutProperty->bIsRequired = InSchema->bRequired.Get(false);
		OutProperty->BindToTypeInfo();

		// Add enum as it's own model, and reference it as this properties type
		if (InSchema->Enum.IsSet() && !InSchema->Enum->IsEmpty())
		{
			const FWebAPITypeNameVariant EnumTypeName = OutProperty->Type;
			EnumTypeName.TypeInfo->SetName(ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName));

			const UWebAPIEnum* Enum = ConvertEnum(InSchema, EnumTypeName);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitEnumForPropertyOfModel", "Implicit enum created for property \"{PropertyName}\" of model \"{ModelName}\"."),
				TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
				TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
			MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);

			Enum->Name.TypeInfo->DebugString += LogMessage.ToString();

			Enum->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();
			Enum->Name.TypeInfo->SetNested(InModelName);

			OutProperty->Type = Enum->Name;
			OutProperty->Type.TypeInfo->SetModel(Enum);
		}
		// Add struct as it's own model, and reference it as this properties type
		else if (OutProperty->Type.ToString(true).IsEmpty())
		{
			check(OutProperty->Type.HasTypeInfo());
			
			const FWebAPITypeNameVariant ModelTypeName = OutProperty->Type;
			ModelTypeName.TypeInfo->SetName(ProviderSettings.MakeNestedPropertyTypeName(InModelName, InPropertyName));
			ModelTypeName.TypeInfo->Prefix = TEXT("F");
			ModelTypeName.TypeInfo->SetNested(InModelName);

			UWebAPIModel* Model = OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.HasTypeInfo() ? ModelTypeName.TypeInfo.Get() : nullptr);
			ConvertModel(InSchema, {}, Model);

			const FText LogMessage = FText::FormatNamed(
				LOCTEXT("AddedImplicitModelForPropertyOfModel", "Implicit model created for property \"{PropertyName}\" of model \"{ModelName}\"."),
				TEXT("PropertyName"), FText::FromString(*InPropertyName.ToString(true)),
				TEXT("ModelName"), FText::FromString(*InModelName.ToString(true)));
			MessageLog->LogInfo(LogMessage, FWebAPIOpenAPIProvider::LogName);
			
			Model->Name.TypeInfo->DebugString += LogMessage.ToString();
			
			Model->Name.TypeInfo->JsonName = InPropertyName.GetJsonName();
			Model->Name.TypeInfo->JsonType = UWebAPIStaticTypeRegistry::ToFromJsonType;

			OutProperty->Type = Model->Name;
			OutProperty->Type.TypeInfo->SetModel(Model);
		}
		
		OutProperty->Name = ResolvePropertyName(OutProperty, InModelName);

		return true;
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPIProperty> FWebAPIOpenAPISchemaConverter::ConvertProperty(const TSharedPtr<SchemaType>& InSrcSchema, const TObjectPtr<UWebAPIModel>& InModel, const FWebAPINameVariant& InPropertyName, const FString& InDefinitionName)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		const FWebAPINameVariant SrcPropertyName = InPropertyName;
		FString SrcPropertyDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(*InModel->Name.ToString(true), SrcPropertyName);
		const TSharedPtr<OpenAPI::V3::FSchemaObject> SrcPropertyValue = ResolveReference(InSrcSchema, SrcPropertyDefinitionName);

		if (SrcPropertyValue)
		{
			const TObjectPtr<UWebAPIProperty>& DstProperty = InModel->Properties.Add_GetRef(NewObject<UWebAPIProperty>(InModel));
			ConvertProperty(InModel->Name,
				FWebAPINameInfo(NameTransformer(SrcPropertyName), SrcPropertyName.GetJsonName(), InModel->Name),
				SrcPropertyValue,
				SrcPropertyDefinitionName,
				DstProperty);
			
			return DstProperty;
		}

		return nullptr;
	}

	template <typename SchemaType>
	bool FWebAPIOpenAPISchemaConverter::ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel)
	{
		static_assert(TIsDerivedFrom<SchemaType, OpenAPI::V3::FSchemaObjectBase>::Value, "Type is not derived from OpenAPI::V3::FSchemaObjectBase.");

		FWebAPITypeNameVariant ModelTypeName;
		if(InModelTypeName.IsValid())
		{
			ModelTypeName = InModelTypeName;
		}
		else if(OutModel && OutModel->Name.HasTypeInfo())
		{
			ModelTypeName = OutModel->Name;
		}
		else
		{
			const FString ModelName = InSrcSchema->Name;
			check(!ModelName.IsEmpty());

			ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(*ModelName),
				*ModelName,
				TEXT("F"));
		}

		if (!InSrcSchema.IsValid())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ModelName"), FText::FromString(*ModelTypeName.ToString(true)));
			MessageLog->LogWarning(
				FText::Format(LOCTEXT("SchemaInvalid",
						"The schema for model \"{ModelName}\" was invalid/null."),
					Args),
				FWebAPIOpenAPIProvider::LogName);
			return false;
		}

		const TObjectPtr<UWebAPIModel>& Model = OutModel ? OutModel : OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());

		UWebAPIModelBase* ModelBase = Model;
		if (!ConvertModelBase(InSrcSchema, ModelBase))
		{
			return false;
		}

		Model->Name = ModelTypeName;

		if (InSrcSchema->Properties.IsSet() && !InSrcSchema->Properties->IsEmpty())
		{
			for (const TTuple<FString, Json::TJsonReference<OpenAPI::V3::FSchemaObject>>& NamePropertyPair : InSrcSchema->Properties.GetValue())
			{
				const FString& SrcPropertyName = NamePropertyPair.Key;
				
				FString SrcPropertyDefinitionName = ProviderSettings.MakeNestedPropertyTypeName(*ModelTypeName.ToString(true), SrcPropertyName);
				TSharedPtr<OpenAPI::V3::FSchemaObject> SrcPropertySchema = ResolveReference(NamePropertyPair.Value, SrcPropertyDefinitionName);

				TObjectPtr<UWebAPIProperty>& DstProperty = Model->Properties.Add_GetRef(NewObject<UWebAPIProperty>(Model));
				ConvertProperty(Model->Name,
								FWebAPINameInfo(NameTransformer(SrcPropertyName), SrcPropertyName, ModelTypeName),
								SrcPropertySchema,
								SrcPropertyDefinitionName,
								DstProperty);
			}
		}

		Model->BindToTypeInfo();

		return true;
	}

	template <>
	inline bool FWebAPIOpenAPISchemaConverter::ConvertModel<OpenAPI::V3::FParameterObject>(const TSharedPtr<OpenAPI::V3::FParameterObject>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName, UWebAPIModel* OutModel)
	{
		FWebAPITypeNameVariant ModelTypeName;
		if(InModelTypeName.IsValid() && InModelTypeName.HasTypeInfo())
		{
			ModelTypeName = InModelTypeName;
		}
		else if(OutModel && OutModel->Name.HasTypeInfo())
		{
			ModelTypeName = OutModel->Name;
		}
		else
		{
			const FString ModelName = InSrcSchema->Name;
			check(!ModelName.IsEmpty());

			ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(*ModelName),
				*ModelName,
				TEXT("F"));
		}

		if (!InSrcSchema.IsValid())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ModelName"), FText::FromString(*ModelTypeName.ToString(true)));
			MessageLog->LogWarning(
				FText::Format(LOCTEXT("SchemaInvalid",
						"The schema for model \"{ModelName}\" was invalid/null."),
					Args),
				FWebAPIOpenAPIProvider::LogName);
			return false;
		}

		UWebAPIModel* Model = OutModel ? OutModel : OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());

		UWebAPIModelBase* ModelBase = Model;
		if (!ConvertModelBase(InSrcSchema, ModelBase))
		{
			return false;
		}

		Model->Name = ModelTypeName;

		Model->BindToTypeInfo();

		return true;
	}

	template <typename SchemaType>
	TObjectPtr<UWebAPIModel> FWebAPIOpenAPISchemaConverter::ConvertModel(const TSharedPtr<SchemaType>& InSrcSchema, const FWebAPITypeNameVariant& InModelTypeName)
	{
		check(InSrcSchema.IsValid());

		FWebAPITypeNameVariant ModelTypeName;
		if(InModelTypeName.IsValid() && ModelTypeName.HasTypeInfo())
		{
			ModelTypeName = InModelTypeName;
		}
		else
		{
			const FString ModelName = InSrcSchema->Name;
			check(!ModelName.IsEmpty());

			ModelTypeName = OutputSchema->TypeRegistry->GetOrMakeGeneratedType(
				EWebAPISchemaType::Model,
				NameTransformer(*ModelName),
				*ModelName,
				TEXT("F"));
		}
		
		UWebAPIModel* Model = OutputSchema->AddModel<UWebAPIModel>(ModelTypeName.TypeInfo.Get());
		if(ConvertModel(InSrcSchema, ModelTypeName, Model))
		{
			return Model;
		}

		return nullptr;
	}
};

#undef LOCTEXT_NAMESPACE
