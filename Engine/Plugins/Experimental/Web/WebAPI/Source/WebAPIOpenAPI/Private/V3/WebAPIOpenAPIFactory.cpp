// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOpenAPIFactory.h"

#include "V3/WebAPIOpenAPIProvider.h"
#include "WebAPIOpenAPILog.h"
#include "Serialization/JsonSerializer.h"
#include "WebAPIDefinition.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPIFactory"

UWebAPIOpenAPIFactory::UWebAPIOpenAPIFactory()
	: Provider(MakeShared<FWebAPIOpenAPIProvider>())
{
}

bool UWebAPIOpenAPIFactory::CanImportWebAPI(const FString& InFileName, const FString& InFileContents)
{
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(InFileContents), JsonObject))
	{
		return false;
	}

	return JsonObject->HasField(TEXT("openapi"));
}

TFuture<bool> UWebAPIOpenAPIFactory::ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents)
{
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(InFileContents), JsonObject))
	{
		UE_LOGF(LogWebAPIOpenAPI, Error, "Couldn't deserialize file contents as Json");
		return MakeFulfilledPromise<bool>(false).GetFuture();
	}

	// Write raw spec contents
	UWebAPIOpenAPIAssetData* ImportData = InDefinition->AddOrGetImportedDataCache<UWebAPIOpenAPIAssetData>("OpenAPI");
	ImportData->FileContents = InFileContents;

	// Parse spec
	return Provider->ConvertToWebAPISchema(InDefinition)
	.Next([](EWebAPIConversionResult bInConversionResult)
	{
		return bInConversionResult == EWebAPIConversionResult::Succeeded;
	});
}

#undef LOCTEXT_NAMESPACE
