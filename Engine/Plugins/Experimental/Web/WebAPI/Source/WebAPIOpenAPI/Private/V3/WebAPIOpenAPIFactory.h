// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Factories/WebAPIOpenAPIFactoryBase.h"
#include "WebAPIOpenAPIFactory.generated.h"

class FWebAPIOpenAPIProvider;

UCLASS()
class WEBAPIOPENAPI_API UWebAPIOpenAPIAssetData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString FileContents;
};

/** Handles importing for OpenAPI V2 files. */
UCLASS()
class WEBAPIOPENAPI_API UWebAPIOpenAPIFactory
	: public UWebAPIOpenAPIFactoryBase
{
	GENERATED_BODY()

public:
	UWebAPIOpenAPIFactory();

	virtual bool CanImportWebAPI(const FString& InFileName, const FString& InFileContents) override;
	virtual TFuture<bool> ImportWebAPI(UWebAPIDefinition* InDefinition, const FString& InFileName, const FString& InFileContents) override;

protected:
	TSharedPtr<FWebAPIOpenAPIProvider> Provider;
};
