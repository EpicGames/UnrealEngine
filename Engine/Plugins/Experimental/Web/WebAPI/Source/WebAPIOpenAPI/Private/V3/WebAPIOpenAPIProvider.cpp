// Copyright Epic Games, Inc. All Rights Reserved.

#include "V3/WebAPIOpenAPIProvider.h"

#include "Dom/WebAPITypeRegistry.h"
#include "WebAPIOpenAPIFactory.h"
#include "Dom/WebAPISchema.h"
#include "Serialization/JsonSerializer.h"
#include "V3/WebAPIOpenAPIConverter.h"
#include "WebAPIDefinition.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPIProvider"

TFuture<EWebAPIConversionResult> FWebAPIOpenAPIProvider::ConvertToWebAPISchema(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition)
{
	TFuture<EWebAPIConversionResult> FailedWithErrorsResult = MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::FailedWithErrors).GetFuture();
	TFuture<EWebAPIConversionResult> FailedWithWarningsResult = MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::FailedWithWarnings).GetFuture();
	
	if(!InDefinition.IsValid())
	{
		return FailedWithErrorsResult;
	}

	// Get cached file contents from Definition
	const UWebAPIOpenAPIAssetData* AssetData = InDefinition->AddOrGetImportedDataCache<UWebAPIOpenAPIAssetData>(TEXT("OpenAPI"));

	const TSharedPtr<FWebAPIMessageLog> MessageLog = InDefinition->GetMessageLog();

	// Load Json
	TSharedPtr<FJsonObject> JsonObject;
	if(!FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(AssetData->FileContents), JsonObject))
	{
		return FailedWithErrorsResult;
	}

	// Parse OpenAPI JsonObject
	TSharedPtr<UE::WebAPI::OpenAPI::V3::FOpenAPIObject> Root = MakeShared<UE::WebAPI::OpenAPI::V3::FOpenAPIObject>();
	if(!Root->FromJson(JsonObject.ToSharedRef()))
	{
		MessageLog->LogError(FText::FromString(TEXT("Failed to load the OpenAPI schema.")), FWebAPIOpenAPIProvider::LogName);
		return FailedWithErrorsResult;
	}

	// Convert to WebAPI schema
	UWebAPISchema* WebAPISchema = InDefinition->GetWebAPISchema();
	WebAPISchema->TypeRegistry->Clear();
	const TSharedRef<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter, ESPMode::ThreadSafe> Converter = MakeShared<UE::WebAPI::OpenAPI::FWebAPIOpenAPISchemaConverter>(
		Root,
		WebAPISchema,
		MessageLog.ToSharedRef(),
		InDefinition->GetProviderSettings());
	if(Converter->Convert())
	{
		if(!WebAPISchema->TypeRegistry->CheckAllNamed())
		{
			return FailedWithWarningsResult;
		}
		
		return MakeFulfilledPromise<EWebAPIConversionResult>(EWebAPIConversionResult::Succeeded).GetFuture();
	}

	MessageLog->LogError(FText::FromString(TEXT("Failed to convert the OpenAPI schema to the WebAPI schema.")), FWebAPIOpenAPIProvider::LogName);
	return FailedWithErrorsResult;
}

#undef LOCTEXT_NAMESPACE
