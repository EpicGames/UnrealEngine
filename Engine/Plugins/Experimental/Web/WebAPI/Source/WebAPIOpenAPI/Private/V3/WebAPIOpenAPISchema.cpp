// Copyright Epic Games, Inc. All Rights Reserved.

#include "V3/WebAPIOpenAPISchema.h"


namespace UE::WebAPI::OpenAPI::V3
{
	bool FSchemaObjectBase::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("required"), bRequired);
		bSetAField |= Json::TryGetField(InJson, TEXT("deprecated"), bDeprecated);
		bSetAField |= Json::TryGetField(InJson, TEXT("example"), Example);
		return bSetAField;
	}

	bool FParameterObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = Super::FromJson(InJson);
		bSetAField |= Json::TryGetField(InJson, TEXT("in"), In);
		bSetAField |= Json::TryGetField(InJson, TEXT("allowEmptyValue"), bAllowEmptyValue);
		bSetAField |= Json::TryGetField(InJson, TEXT("style"), Style);
		bSetAField |= Json::TryGetField(InJson, TEXT("explode"), bExplode);
		bSetAField |= Json::TryGetField(InJson, TEXT("allowReserved"), bAllowReserved);
		bSetAField |= Json::TryGetField(InJson, TEXT("schema"), Schema);
		bSetAField |= Json::TryGetField(InJson, TEXT("examples"), Examples);
		bSetAField |= Json::TryGetField(InJson, TEXT("content"), Content);
		return bSetAField;
	}
	
	bool FComponentsObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("schemas"), Schemas);
		bSetAField |= Json::TryGetField(InJson, TEXT("responses"), Responses);
		bSetAField |= Json::TryGetField(InJson, TEXT("parameters"), Parameters);
		bSetAField |= Json::TryGetField(InJson, TEXT("examples"), Examples);
		bSetAField |= Json::TryGetField(InJson, TEXT("requestBodies"), RequestBodies);
		bSetAField |= Json::TryGetField(InJson, TEXT("headers"), Headers);
		bSetAField |= Json::TryGetField(InJson, TEXT("securitySchemes"), SecuritySchemes);
		bSetAField |= Json::TryGetField(InJson, TEXT("links"), Links);
		bSetAField |= Json::TryGetField(InJson, TEXT("callbacks"), Callbacks);
		return bSetAField;
	}

	bool FContactObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), Url);
		bSetAField |= Json::TryGetField(InJson, TEXT("email"), Email);
		return bSetAField;
	}

	bool FDiscriminatorObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("propertyName"), PropertyName);
		bSetAField |= Json::TryGetField(InJson, TEXT("mapping"), Mapping);
		return bSetAField;
	}

	bool FEncodingObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("contentType"), ContentType);
		bSetAField |= Json::TryGetField(InJson, TEXT("headers"), Headers);
		bSetAField |= Json::TryGetField(InJson, TEXT("style"), Style);
		bSetAField |= Json::TryGetField(InJson, TEXT("explode"), bExplode);
		bSetAField |= Json::TryGetField(InJson, TEXT("allowReserved"), bAllowReserved);
		return bSetAField;
	}

	bool FExampleObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("summary"), Summary);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		// bSetAField |= Json::TryGetField(InJson, TEXT("value"), Value);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalValue"), ExternalValue);
		return bSetAField;
	}

	bool FExternalDocumentationObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), Url);
		return bSetAField;
	}

	bool FHeaderObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		const bool bSetAField = Super::FromJson(InJson);
		return bSetAField;
	}

	bool FInfoObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("title"), Title);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("termsOfService"), TermsOfService);
		bSetAField |= Json::TryGetField(InJson, TEXT("contact"), Contact);
		bSetAField |= Json::TryGetField(InJson, TEXT("license"), License);
		bSetAField |= Json::TryGetField(InJson, TEXT("version"), Version);
		return bSetAField;
	}

	bool FLicenseObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), Url);
		return bSetAField;
	}

	bool FLinkObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("operationRef"), OperationRef);
		bSetAField |= Json::TryGetField(InJson, TEXT("operationId"), OperationId);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("server"), Server);
		return bSetAField;
	}

	bool FMediaTypeObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("schema"), Schema);
		bSetAField |= Json::TryGetField(InJson, TEXT("example"), Example);
		bSetAField |= Json::TryGetField(InJson, TEXT("examples"), Examples);
		bSetAField |= Json::TryGetField(InJson, TEXT("encoding"), Encoding);
		return bSetAField;
	}

	bool FOAuthFlowObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("authorizationUrl"), AuthorizationUrl);
		bSetAField |= Json::TryGetField(InJson, TEXT("tokenUrl"), TokenUrl);
		bSetAField |= Json::TryGetField(InJson, TEXT("refreshUrl"), RefreshUrl);
		bSetAField |= Json::TryGetField(InJson, TEXT("scopes"), Scopes);
		return bSetAField;
	}

	bool FOAuthFlowsObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("implicit"), Implicit);
		bSetAField |= Json::TryGetField(InJson, TEXT("password"), Password);
		bSetAField |= Json::TryGetField(InJson, TEXT("clientCredentials"), ClientCredentials);
		bSetAField |= Json::TryGetField(InJson, TEXT("authorizationCode"), AuthorizationCode);
		return bSetAField;
	}

	bool FOpenAPIObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("openapi"), OpenAPI);
		bSetAField |= Json::TryGetField(InJson, TEXT("info"), Info);
		bSetAField |= Json::TryGetField(InJson, TEXT("servers"), Servers);
		bSetAField |= Json::TryGetField(InJson, TEXT("paths"), Paths);
		bSetAField |= Json::TryGetField(InJson, TEXT("components"), Components);
		bSetAField |= Json::TryGetField(InJson, TEXT("security"), Security);
		bSetAField |= Json::TryGetField(InJson, TEXT("tags"), Tags);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		return bSetAField;
	}

	bool FOperationObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("tags"), Tags);
		bSetAField |= Json::TryGetField<FString>(InJson, TEXT("summary"), Summary);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		bSetAField |= Json::TryGetField(InJson, TEXT("operationId"), OperationId);
		bSetAField |= Json::TryGetField(InJson, TEXT("parameters"), Parameters);
		bSetAField |= Json::TryGetField(InJson, TEXT("requestBody"), RequestBody);
		bSetAField |= Json::TryGetField(InJson, TEXT("responses"), Responses);
		bSetAField |= Json::TryGetField(InJson, TEXT("callbacks"), Callbacks);
		bSetAField |= Json::TryGetField(InJson, TEXT("deprecated"), bDeprecated);
		bSetAField |= Json::TryGetField(InJson, TEXT("security"), Security);
		bSetAField |= Json::TryGetField(InJson, TEXT("servers"), Servers);
		return bSetAField;
	}

	bool FReferenceObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("$ref"), $Ref);
		return bSetAField;
	}

	bool FRequestBodyObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("content"), Content);
		bSetAField |= Json::TryGetField(InJson, TEXT("required"), bRequired);
		return bSetAField;
	}

	bool FResponseObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("headers"), Headers);
		bSetAField |= Json::TryGetField(InJson, TEXT("content"), Content);
		return bSetAField;
	}

	bool FSchemaObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = Super::FromJson(InJson);
		
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("format"), Format);
		bSetAField |= Json::TryGetField(InJson, TEXT("default"), Default);
		bSetAField |= Json::TryGetField(InJson, TEXT("allOf"), AllOf);
		bSetAField |= Json::TryGetField(InJson, TEXT("oneOf"), OneOf);
		bSetAField |= Json::TryGetField(InJson, TEXT("anyOf"), AnyOf);
		bSetAField |= Json::TryGetField(InJson, TEXT("not"), Not);

		bSetAField |= Json::TryGetField(InJson, TEXT("title"), Title);
		bSetAField |= Json::TryGetField(InJson, TEXT("multipleOf"), MultipleOf);
		bSetAField |= Json::TryGetField(InJson, TEXT("minimum"), Minimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMinimum"), bExclusiveMinimum);
		bSetAField |= Json::TryGetField(InJson, TEXT("maximum"), Maximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("exclusiveMaximum"), bExclusiveMaximum);
		bSetAField |= Json::TryGetField(InJson, TEXT("minLength"), MinLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxLength"), MaxLength);
		bSetAField |= Json::TryGetField(InJson, TEXT("pattern"), Pattern);
		bSetAField |= Json::TryGetField(InJson, TEXT("minItems"), MinItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxItems"), MaxItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("uniqueItems"), bUniqueItems);
		bSetAField |= Json::TryGetField(InJson, TEXT("minProperties"), MinProperties);
		bSetAField |= Json::TryGetField(InJson, TEXT("maxProperties"), MaxProperties);
		bSetAField |= Json::TryGetField(InJson, TEXT("enum"), Enum);
		bSetAField |= Json::TryGetField(InJson, TEXT("items"), Items);
		bSetAField |= Json::TryGetField(InJson, TEXT("nullable"), bNullable);
		bSetAField |= Json::TryGetField(InJson, TEXT("discriminator"), Discriminator);
		bSetAField |= Json::TryGetField(InJson, TEXT("readOnly"), bReadOnly);
		bSetAField |= Json::TryGetField(InJson, TEXT("writeOnly"), bWriteOnly);
		bSetAField |= Json::TryGetField(InJson, TEXT("xml"), Xml);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		bSetAField |= Json::TryGetField(InJson, TEXT("properties"), Properties);
		
		return bSetAField;
	}

	bool FSecuritySchemeObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("type"), Type);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("in"), In);
		bSetAField |= Json::TryGetField(InJson, TEXT("scheme"), Scheme);
		bSetAField |= Json::TryGetField(InJson, TEXT("bearerFormat"), BearerFormat);
		bSetAField |= Json::TryGetField(InJson, TEXT("flows"), Flows);
		bSetAField |= Json::TryGetField(InJson, TEXT("openIdConnectUrl"), OpenIdConnectUrl);		
		return bSetAField;
	}

	bool FServerObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("url"), Url);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("variables"), Variables);
		return bSetAField;
	}

	bool FServerVariableObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("enum"), Enum);
		bSetAField |= Json::TryGetField(InJson, TEXT("default"), Default);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		return bSetAField;
	}

	bool FTagObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("externalDocs"), ExternalDocs);
		return bSetAField;
	}

	bool FXMLObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("name"), Name);
		bSetAField |= Json::TryGetField(InJson, TEXT("namespace"), Namespace);
		bSetAField |= Json::TryGetField(InJson, TEXT("prefix"), Prefix);
		bSetAField |= Json::TryGetField(InJson, TEXT("attribute"), bAttribute);
		bSetAField |= Json::TryGetField(InJson, TEXT("wrapped"), bWrapped);
		return bSetAField;
	}

	// https://swagger.io/specification/#path-item-object
	bool FPathItemObject::FromJson(const TSharedRef<FJsonObject>& InJson)
	{
		// Describes the operations available on a single path. A Path Item MAY be empty,
		// due to ACL constraints. The path itself is still exposed to the documentation viewer
		// but they will not know which operations and parameters are available.
		bool bSetAField = false;
		bSetAField |= Json::TryGetField(InJson, TEXT("summary"), Summary);
		bSetAField |= Json::TryGetField(InJson, TEXT("description"), Description);
		bSetAField |= Json::TryGetField(InJson, TEXT("get"), Get);
		bSetAField |= Json::TryGetField(InJson, TEXT("put"), Put);
		bSetAField |= Json::TryGetField(InJson, TEXT("post"), Post);
		bSetAField |= Json::TryGetField(InJson, TEXT("delete"), Delete);
		bSetAField |= Json::TryGetField(InJson, TEXT("options"), Options);
		bSetAField |= Json::TryGetField(InJson, TEXT("head"), Head);
		bSetAField |= Json::TryGetField(InJson, TEXT("patch"), Patch);
		bSetAField |= Json::TryGetField(InJson, TEXT("trace"), Trace);
		return bSetAField;
	}
}

bool UE::Json::FromJson(const TSharedRef<FJsonObject>& InJsonObject, UE::WebAPI::OpenAPI::V3::FPathsObject& OutObject)
{
	return UE::Json::TryGet(MakeShared<FJsonValueObject>(InJsonObject), OutObject);
}

bool UE::Json::FromJson(const TSharedRef<FJsonObject>& InJsonObject, UE::WebAPI::OpenAPI::V3::FSecurityRequirementObject& OutObject)
{
	return UE::Json::TryGet(MakeShared<FJsonValueObject>(InJsonObject), OutObject);
}
