// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebAPIOpenAPIModule.h"

#include "IWebAPIEditorModule.h"
#include "V2/WebAPISwaggerProvider.h"
#include "V3/WebAPIOpenAPIProvider.h"

#define LOCTEXT_NAMESPACE "WebAPIOpenAPI"

void FWebAPIOpenAPIModule::StartupModule()
{
	IWebAPIEditorModuleInterface& WebAPIEditorModule = IWebAPIEditorModuleInterface::Get();

	const TSharedRef<FWebAPISwaggerProvider> SwaggerAdapter = MakeShared<FWebAPISwaggerProvider>();
	WebAPIEditorModule.AddProvider(TEXT("Swagger"), SwaggerAdapter);

	const TSharedRef<FWebAPIOpenAPIProvider> OpenAPIAdapter = MakeShared<FWebAPIOpenAPIProvider>();
	WebAPIEditorModule.AddProvider(TEXT("OpenAPI"), OpenAPIAdapter);
}

void FWebAPIOpenAPIModule::ShutdownModule()
{
	static FName WebAPIEditorModuleName = TEXT("WebAPIEditorModule");
	if(FModuleManager::Get().IsModuleLoaded(WebAPIEditorModuleName))
	{
		IWebAPIEditorModuleInterface& WebAPIEditorModule = IWebAPIEditorModuleInterface::Get();
		WebAPIEditorModule.RemoveProvider(TEXT("Swagger"));
		WebAPIEditorModule.RemoveProvider(TEXT("OpenAPI"));
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebAPIOpenAPIModule, WebAPIOpenAPI)
