// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebAPIOpenAPIModule.h"


class FWebAPIOpenAPIModule final
    : public IWebAPIOpenAPIModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
};
