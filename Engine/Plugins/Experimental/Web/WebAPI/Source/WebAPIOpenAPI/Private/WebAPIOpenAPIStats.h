// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("WebAPIOpenAPI"), STATGROUP_WebAPIOpenAPI, STATCAT_Advanced);
