// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIDefinition.h"
#include "WebAPIProvider.h"
#include "Async/Future.h"
#include "V2/WebAPISwaggerSchema.h"

/** */
class WEBAPIOPENAPI_API FWebAPISwaggerProvider
	: public IWebAPIProviderInterface
{
public:
	virtual TFuture<EWebAPIConversionResult> ConvertToWebAPISchema(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition) override;

public:
	static constexpr const TCHAR* LogName = TEXT("Swagger Provider");
};
