// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIJsonUtilities.h"
#include "Dom/JsonObject.h"

namespace UE
{
	namespace WebAPI
	{
		namespace OpenAPI
		{
			// OpenAPI 2.0/Swagger Spec Schema
			// https://swagger.io/docs/specification/2-0/basic-structure/
			namespace V2
			{
				class FInfo;
				class FOperation;
				class FSchema;

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#referenceObject */
				class FReference
				{
				public:
					TSharedPtr<FString> $Ref;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#itemsObject */
				class FItems
				{
				public:
					FString Type; // will map to EJson enum, if it's string AND enum is set, it's an enum by name
					TOptional<FString> Format;
					TOptional<TSharedPtr<FItems>> Items;
					TOptional<FString> CollectionFormat; // csv, ssv, tsv, pipes
					TOptional<FString> Default;
					TOptional<float> Maximum;
					TOptional<float> ExclusiveMaximum;
					TOptional<float> Minimum;
					TOptional<float> ExclusiveMinimum;
					TOptional<int32> MaxLength;
					TOptional<int32> MinLength;
					TOptional<FString> Pattern;
					TOptional<int32> MaxItems;
					TOptional<int32> MinItems;
					TOptional<bool> bUniqueItems; // Array should be a Set (of unique values)
					TOptional<TArray<FString>> Enum; // All possible values
					TOptional<int32> MultipleOf;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#externalDocumentationObject */
				class FExternalDocumentation
				{
				public:
					TOptional<FString> Description;
					FString Url;
 
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#xmlObject */
				class FXML
				{
				public:
					TOptional<FString> Name;
					TOptional<FString> Namespace;
					TOptional<FString> Prefix;
					TOptional<bool> Attribute;
					TOptional<bool> Wrapped;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				class FSchemaBase
				{
				public:
					FString Name;
					TOptional<FString> Description;
					TOptional<FString> Type;
					TOptional<FString> Format; // 'int32' | 'int64' | 'float' | 'double' | 'string' | 'boolean' | 'byte' | 'binary' | 'date' | 'date-time' | 'password';
					TOptional<FString> Default;
					TOptional<bool> bRequired;
					TOptional<float> Minimum;
					TOptional<bool> bExclusiveMinimum;
					TOptional<float> Maximum;
					TOptional<bool> bExclusiveMaximum;					
					TOptional<int32> MinLength;
					TOptional<int32> MaxLength;
					TOptional<FString> Pattern;					
					TOptional<int32> MinItems;
					TOptional<int32> MaxItems;
					TOptional<bool> bUniqueItems; // Array should be a Set (of unique values)
					TOptional<TArray<FString>> Enum; // All possible values
					TOptional<Json::TJsonReference<FSchema>> Items; // Item type (if this is an array)

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#schemaObject */
				class FSchema : public FSchemaBase
				{
				public:
					using Super = FSchemaBase;

					TOptional<FString> Title;
					TOptional<int32> MultipleOf;
					TOptional<int32> MaxProperties;
					TOptional<int32> MinProperties;
					
					TOptional<TArray<Json::TJsonReference<FSchema>>> AllOf;
					TOptional<TMap<FString, Json::TJsonReference<FSchema>>> Properties;
					TOptional<FString> AdditionalProperties;
					TOptional<FString> Discriminator;
					TOptional<bool> bReadOnly;
					TOptional<TSharedPtr<FXML>> XML;
					TOptional<TSharedPtr<FExternalDocumentation>> ExternalDocs;
					TOptional<FString> Example;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#parameterObject */
				class FParameter : public FSchemaBase
				{
				public:
					using Super = FSchemaBase;

					FString In; // One of query, header, path, formData or body  'path' | 'query' | 'header' | 'formData' | 'body';
					TOptional<FString> Description;
					TOptional<Json::TJsonReference<FSchema>> Schema;	
					TOptional<bool> bAllowEmptyValue;
					// TSharedPtr<FItems> Items;
					TOptional<FString> CollectionFormat; // 'csv' | 'ssv' | 'tsv' | 'pipes' | 'multi';
					TOptional<int32> MultipleOf;
					
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#headerObject */
				class FHeader
				{
				public:
					TOptional<FString> Description;
					FString Type;
					TOptional<FString> Format;
					TSharedPtr<FItems> Items;
					TOptional<FString> CollectionFormat;
					TOptional<FString> Default;
					TOptional<float> Maximum;
					TOptional<bool> bExclusiveMaximum;
					TOptional<float> Minimum;
					TOptional<bool> bExclusiveMinimum;
					TOptional<int32> MaxLength;
					TOptional<int32> MinLength;
					TOptional<FString> Pattern;
					TOptional<int32> MaxItems;
					TOptional<int32> MinItems;
					TOptional<bool> bUniqueItems; // Array should be a Set (of unique values)
					TOptional<TArray<FString>> Enum;
					TOptional<int32> MultipleOf;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#exampleObject */
				class FExample : public TMap<FString, FString>
				{
				public:
					// Key is mime-type
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#responseObject */
				class FResponse
				{
				public:
					TOptional<FString> Name;
					FString Description;
					TOptional<Json::TJsonReference<FSchema>> Schema;
					TOptional<TMap<FString, TSharedPtr<FHeader>>> Headers;
					TOptional<TSharedPtr<FExample>> Examples;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				class FScopes : public TMap<FString, FString>
				{
				public:
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#securitySchemeObject */
				class FSecurityScheme
				{
				public:
					TOptional<FString> Name;
					FString Type; // 'basic' | 'apiKey' | 'oauth2';
					TOptional<FString> Description;					
					TOptional<FString> In; // 'query' | 'header';
					TOptional<FString> Flow; // 'implicit' | 'password' | 'application' | 'accessCode';
					TOptional<FString> AuthorizationUrl;
					TOptional<FString> TokenUrl;
					TArray<TMap<FString, TSharedPtr<FScopes>>> Scopes;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#securityRequirementObject */
				class FSecurityRequirement : public TMap<FString, TArray<FString>>
				{
				public:
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#tagObject */
				class FTag
				{
				public:
					FString Name;
					TOptional<FString> Description;
					TOptional<TSharedPtr<FExternalDocumentation>> ExternalDocs;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#pathItemObject */
				class FPath
				{
				public:
					TOptional<FString> $Ref;
					TSharedPtr<FOperation> Get;
					TSharedPtr<FOperation> Put;
					TSharedPtr<FOperation> Post;
					TSharedPtr<FOperation> Delete;
					TSharedPtr<FOperation> Options;
					TSharedPtr<FOperation> Head;
					TSharedPtr<FOperation> Patch;
					TOptional<TArray<Json::TJsonReference<FParameter>>> Parameters;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://github.com/OAI/OpenAPI-Specification/blob/main/versions/2.0.md#swagger-object
				class FSwagger
				{
				public:
					FString Swagger;
					TSharedPtr<FInfo> Info;
					TOptional<FString> Host;
					TOptional<FString> BasePath;
					TOptional<TArray<FString>> Schemes;
					TOptional<TArray<FString>> Consumes; // mimetypes
					TOptional<TArray<FString>> Produces; // mimetypes
					TMap<FString, TSharedPtr<FPath>> Paths;
					TOptional<TMap<FString, TSharedPtr<FSchema>>> Definitions;
					TOptional<TMap<FString, TSharedPtr<FParameter>>> Parameters;
					TOptional<TMap<FString, TSharedPtr<FResponse>>> Responses;
					TOptional<TMap<FString, TSharedPtr<FSecurityScheme>>> SecurityDefinitions;
					TOptional<TArray<TSharedPtr<FSecurityRequirement>>> Security;
					TOptional<TArray<TSharedPtr<FTag>>> Tags;
					TOptional<TSharedPtr<FExternalDocumentation>> ExternalDocs;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#licenseObject */
				class FLicense
				{
				public:
					FString Name;
					TOptional<FString> URL;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#contactObject */
				class FContact
				{
				public:
					TOptional<FString> Name;
					TOptional<FString> URL;
					TOptional<FString> Email;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/main/versions/2.0.md#infoObject */
				class FInfo
				{
				public:
					FString Title;
					TOptional<FString> Description;
					TOptional<FString> TermsOfService;
					TOptional<FString> Contact;
					TOptional<TSharedPtr<FLicense>> License;
					FString Version;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				/** https://github.com/OAI/OpenAPI-Specification/blob/master/versions/2.0.md#operationObject */
				class FOperation
				{
				public:
					TOptional<TArray<FString>> Tags;
					TOptional<FString> Summary;
					TOptional<FString> Description;
					TOptional<TSharedPtr<FExternalDocumentation>> ExternalDocs;
					TOptional<FString> OperationId;
					TOptional<TArray<FString>> Consumes; // mimetypes
					TOptional<TArray<FString>> Produces; // mimetypes
					TOptional<TArray<Json::TJsonReference<FParameter>>> Parameters;
					TMap<FString, TSharedPtr<FResponse>>  Responses;
					TArray<FString> Schemes; //  ('http' | 'https' | 'ws' | 'wss')[];
					TOptional<bool> bDeprecated;
					TOptional<TArray<TSharedPtr<FSecurityRequirement>>> Security;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};
			}
		}
	}
}
