// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "WebAPIProvider.h"

/** */
class WEBAPIOPENAPI_API FWebAPIOpenAPIProvider
	: public IWebAPIProviderInterface
{
public:
	virtual TFuture<EWebAPIConversionResult> ConvertToWebAPISchema(const TWeakObjectPtr<UWebAPIDefinition>& InDefinition) override;

public:
	static constexpr const TCHAR* LogName = TEXT("OpenAPI Provider");
};
