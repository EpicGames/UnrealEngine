// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebAPIJsonUtilities.h"
#include "Dom/JsonObject.h"

namespace UE
{
	namespace WebAPI
	{
		namespace OpenAPI
		{
			// OpenAPI 3.0 Spec Schema
			namespace V3
			{
				// root document MUST contain "openapi" field with version, ie. "3.0", OR if earlier version "swagger"
				// $ref, %id, $schema
				// null not supported as a type
				// primitives MAY have modifier property "format", format might be "email" "uuid", etc.
				// formats defined be spec are ...
				// All Urls MAY be relative references

				class FComponentsObject;
				class FContactObject;
				class FEncodingObject;
				class FExampleObject;
				class FExternalDocumentationObject;
				class FHeaderObject;
				class FInfoObject;
				class FLicenseObject;
				class FLinkObject;
				class FMediaTypeObject;
				class FOAuthFlowObject;
				class FOAuthFlowsObject;
				class FOperationObject;
				class FParameterObject;
				class FPathItemObject;
				class FReferenceObject;
				class FRequestBodyObject;
				class FResponseObject;
				class FSchemaObject;
				class FSecuritySchemeObject;
				class FServerObject;
				class FServerVariableObject;
				class FTagObject;
				class FDiscriminatorObject;

				// https://swagger.io/specification/#paths-object
				// all keys must be relative, and with a leading forward slash
				// MAY be extended with Specification Extensions
				using FPathsObject = TMap<FString, TSharedPtr<FPathItemObject>>;

				// https://swagger.io/specification/#callback-object
				// MAY be extended with Specification Extensions
				using FCallbackObject = TMap<FString, FPathItemObject>;

				// https://swagger.io/specification/#security-requirement-object
				// MAY be extended with Specification Extensions
				using FSecurityRequirementObject = TMap<FString, TArray<FString>>;

				class FSchemaObjectBase
				{
				public:
					FString Name;
					TOptional<FString> Description;
					TOptional<bool> bRequired; // REQUIRED IF "in" i  "path"
					TOptional<bool> bDeprecated;
					TOptional<FString> Example;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#parameter-object
				class FParameterObject : public FSchemaObjectBase
				{
				public:
					using Super = FSchemaObjectBase;
					
					FString In; // One of query, header, path or cookie

					TOptional<bool> bAllowEmptyValue;

					TOptional<FString> Style;
					TOptional<bool> bExplode; // if array or object, parameter-per-item, not array/object-as-parameter
					TOptional<bool> bAllowReserved; // allows: :/?#[]@!$&'()*+,;=
					
					TOptional<Json::TJsonReference<FSchemaObject>> Schema;					
					TOptional<TMap<FString, Json::TJsonReference<FExampleObject>>> Examples;

					TOptional<TMap<FString, TSharedPtr<FMediaTypeObject>>> Content; // only one item

					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#schema-object
				class FSchemaObject : public FSchemaObjectBase
				{
				public:
					using Super = FSchemaObjectBase;
					
					// MAY be extended with Specification Extensions
					TOptional<FString> Type;
					TOptional<FString> Format; // 'int32' | 'int64' | 'float' | 'double' | 'string' | 'boolean' | 'byte' | 'binary' | 'date' | 'date-time' | 'password';
					TOptional<FString> Default;
					TOptional<TArray<Json::TJsonReference<FSchemaObject>>> AllOf;
					TOptional<TArray<Json::TJsonReference<FSchemaObject>>> OneOf;
					TOptional<TArray<Json::TJsonReference<FSchemaObject>>> AnyOf;
					TOptional<TArray<Json::TJsonReference<FSchemaObject>>> Not;

					TOptional<FString> Title;
					TOptional<int32> MultipleOf;
					TOptional<float> Minimum;
					TOptional<bool> bExclusiveMinimum;
					TOptional<float> Maximum;
					TOptional<bool> bExclusiveMaximum;
					TOptional<int32> MinLength;
					TOptional<int32> MaxLength;
					TOptional<FString> Pattern;
					TOptional<int32> MinItems;
					TOptional<int32> MaxItems;
					TOptional<bool> bUniqueItems; // Array should be a Set
					TOptional<int32> MinProperties;
					TOptional<int32> MaxProperties;
					TOptional<TArray<FString>> Enum; // All possible values
					TOptional<Json::TJsonReference<FSchemaObject>> Items; // Item type (if this is an array)
					TOptional<bool> bNullable;
					TOptional<Json::TJsonReference<FDiscriminatorObject>> Discriminator;
					TOptional<bool> bReadOnly;
					TOptional<bool> bWriteOnly;
					TOptional<TSharedPtr<FString>> Xml;
					TOptional<TSharedPtr<FExternalDocumentationObject>> ExternalDocs;
					TSharedPtr<FJsonObject> Example;

					TOptional<TMap<FString, Json::TJsonReference<FSchemaObject>>> Properties;
					TOptional<FString> AdditionalProperties;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#path-item-object
				class FPathItemObject
				{
				public:
					TOptional<FString> $Ref;
					TOptional<FString> Summary;
					TOptional<FString> Description;
					TSharedPtr<FOperationObject> Get;
					TSharedPtr<FOperationObject> Put;
					TSharedPtr<FOperationObject> Post;
					TSharedPtr<FOperationObject> Delete;
					TSharedPtr<FOperationObject> Options;
					TSharedPtr<FOperationObject> Head;
					TSharedPtr<FOperationObject> Patch;
					TSharedPtr<FOperationObject> Trace;
					TOptional<TArray<Json::TJsonReference<FServerObject>>> Servers;
					TArray<Json::TJsonReference<FParameterObject>> Parameters;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#components-object
				class FComponentsObject
				{
				public:
					// all keys should match regex ^[a-zA-Z0-9\.\-_]+$
					TMap<FString, Json::TJsonReference<FSchemaObject>> Schemas;
					TMap<FString, Json::TJsonReference<FResponseObject>> Responses;
					TMap<FString, Json::TJsonReference<FParameterObject>> Parameters;
					TMap<FString, Json::TJsonReference<FExampleObject>> Examples;
					TMap<FString, Json::TJsonReference<FRequestBodyObject>> RequestBodies;
					TMap<FString, Json::TJsonReference<FHeaderObject>> Headers;
					TMap<FString, Json::TJsonReference<FSecuritySchemeObject>> SecuritySchemes;
					TMap<FString, Json::TJsonReference<FLinkObject>> Links;
					TMap<FString, Json::TJsonReference<FCallbackObject>> Callbacks;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};
		
				// https://swagger.io/specification/#contact-object
				class FContactObject
				{
				public:
					TOptional<FString> Name;
					TOptional<FString> Url;
					TOptional<FString> Email;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#discriminator-object
				class FDiscriminatorObject
				{
				public:
					FString PropertyName;
					TOptional<TMap<FString, FString>> Mapping;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#encoding-object
				class FEncodingObject
				{
				public:
					TOptional<FString> ContentType;
					TOptional<TMap<FString, Json::TJsonReference<FHeaderObject>>> Headers;
					TOptional<FString> Style;
					TOptional<bool> bExplode;
					TOptional<bool> bAllowReserved;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#example-object
				class FExampleObject
				{
				public:
					TOptional<FString> Summary;
					TOptional<FString> Description;
					TSharedPtr<FJsonObject> Value;
					TOptional<FString> ExternalValue;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};
		
				// https://swagger.io/specification/#external-documentation-object
				class FExternalDocumentationObject
				{
				public:
					TOptional<FString> Description;
					FString Url;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};
		
				// https://swagger.io/specification/#header-object
				class FHeaderObject : public FParameterObject
				{
				public:
					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#info-object
				class FInfoObject
				{
				public:
					FString Title;
					TOptional<FString> Description;
					TOptional<FString> TermsOfService;
					TSharedPtr<FContactObject> Contact;
					TSharedPtr<FLicenseObject> License;
					FString Version;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#license-object
				class FLicenseObject
				{
				public:
					FString Name;
					TOptional<FString> Url;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#link-object
				class FLinkObject
				{
				public:
					TOptional<FString> OperationRef;
					TOptional<FString> OperationId; // mutually exclusive with OperationRef
					//TOptional<TMap<FString, TSharedPtr<TVariant<FVariant, FVariant>>>> Parameters; // @todo: runtime expression object
					//TSharedPtr<TVariant<FVariant, FVariant>> RequestBody; // @todo
					TOptional<FString> Description;
					TSharedPtr<FServerObject> Server;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#media-type-object
				class FMediaTypeObject
				{
				public:
					Json::TJsonReference<FSchemaObject> Schema;
					TOptional<FString> Example;
					TOptional<TMap<FString, Json::TJsonReference<FExampleObject>>> Examples;
					TOptional<TMap<FString, TSharedPtr<FEncodingObject>>> Encoding;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#oauth-flow-object
				class FOAuthFlowObject
				{
				public:
					FString AuthorizationUrl;
					FString TokenUrl;
					TOptional<FString> RefreshUrl;
					TMap<FString, FString> Scopes;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#oauth-flows-object
				class FOAuthFlowsObject
				{
				public:
					TSharedPtr<FOAuthFlowObject> Implicit;
					TSharedPtr<FOAuthFlowObject> Password;
					TSharedPtr<FOAuthFlowObject> ClientCredentials;
					TSharedPtr<FOAuthFlowObject> AuthorizationCode;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#openapi-object
				class FOpenAPIObject
				{
				public:
					FString OpenAPI;
					TSharedPtr<FInfoObject> Info;
					TOptional<TArray<TSharedPtr<FServerObject>>> Servers; // can be empty
					TMap<FString, TSharedPtr<FPathItemObject>> Paths;
					TSharedPtr<FComponentsObject> Components;
					TOptional<TArray<TSharedPtr<FSecurityRequirementObject>>> Security;
					TOptional<TArray<TSharedPtr<FTagObject>>> Tags;
					TOptional<TSharedPtr<FExternalDocumentationObject>> ExternalDocs;

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#operation-object
				class FOperationObject
				{
				public:
					TOptional<TArray<FString>> Tags;
					TOptional<FString> Summary;
					TOptional<FString> Description;
					TSharedPtr<FExternalDocumentationObject> ExternalDocs;
					TOptional<FString> OperationId;
					TArray<Json::TJsonReference<FParameterObject>> Parameters;
					Json::TJsonReference<FRequestBodyObject> RequestBody;
					TMap<FString, Json::TJsonReference<FResponseObject>> Responses;
					TMap<FString, Json::TJsonReference<FCallbackObject>> Callbacks;
					TOptional<bool> bDeprecated;
					TArray<TSharedPtr<FSecurityRequirementObject>> Security;
					TArray<TSharedPtr<FServerObject>> Servers;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#reference-object
				class FReferenceObject
				{
				public:
					FString $Ref;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#request-body-object
				class FRequestBodyObject
				{
				public:
					TOptional<FString> Description;
					TMap<FString, TSharedPtr<FMediaTypeObject>> Content;
					TOptional<bool> bRequired;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#response-object
				class FResponseObject
				{
				public:
					TOptional<FString> Name;
					FString Description;
					TOptional<TMap<FString, Json::TJsonReference<FHeaderObject>>> Headers;
					TOptional<TMap<FString, Json::TJsonReference<FMediaTypeObject>>> Content;
					
					// @todo: patterned fields
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#security-scheme-object
				class FSecuritySchemeObject
				{
				public:
					FString Type; // any of apiKey, http, oauth2, openIdConnect
					TOptional<FString> Description;
					FString Name;
					FString In;
					FString Scheme;			
					TOptional<FString> BearerFormat;
					TSharedPtr<FOAuthFlowsObject> Flows;
					TOptional<FString> OpenIdConnectUrl;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#server-object
				class FServerObject
				{
				public:
					FString Url;
					TOptional<FString> Description;
					TOptional<TMap<FString, TSharedPtr<FServerVariableObject>>> Variables;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#server-variable-object
				class FServerVariableObject
				{
				public:
					TArray<FString> Enum; // 1 or items
					FString Default;
					TOptional<FString> Description;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#tag-object
				class FTagObject
				{
				public:
					FString Name;
					TOptional<FString> Description;
					TOptional<TSharedPtr<FExternalDocumentationObject>> ExternalDocs;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};

				// https://swagger.io/specification/#xml-object
				class FXMLObject
				{
				public:
					TOptional<FString> Name;
					TOptional<FString> Namespace;
					TOptional<FString> Prefix;
					TOptional<bool> bAttribute;
					TOptional<bool> bWrapped;
					// MAY be extended with Specification Extensions

					bool FromJson(const TSharedRef<FJsonObject>& InJson);
				};
			}
		}
	}

	namespace Json
	{
		bool FromJson(const TSharedRef<FJsonObject>& InJsonObject, UE::WebAPI::OpenAPI::V3::FPathsObject& OutObject);

		// bool FromJson(const TSharedRef<FJsonObject>& InJsonObject, UE::WebAPI::OpenAPI::V3::FCallbackObject& OutObject)
		// {
		// 	return false;
		// }

		bool FromJson(const TSharedRef<FJsonObject>& InJsonObject, UE::WebAPI::OpenAPI::V3::FSecurityRequirementObject& OutObject);
	}
}
