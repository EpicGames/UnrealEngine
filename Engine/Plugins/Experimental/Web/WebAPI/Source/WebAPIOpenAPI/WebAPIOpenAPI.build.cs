// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebAPIOpenAPI : ModuleRules
{
	public WebAPIOpenAPI(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"AssetTools",
				"Core",
				"CoreUObject",
				"EditorStyle",
				"EditorWidgets",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"WebAPI",
				"WebAPIEditor"
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Projects"
				});
		}
    }
}
