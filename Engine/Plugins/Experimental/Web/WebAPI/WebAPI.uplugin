{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "WebAPI",
	"Description": "Automated generation of web based APIs",
	"Category": "Web",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebAPI",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
        {
			"Name": "WebAPIEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
        {
            "Name": "WebAPIBlueprintGraph",
            "Type": "UncookedOnly",
            "LoadingPhase": "Default"
        },
        {
			"Name": "WebAPIOpenAPI",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "WebAPILiquidJS",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PluginBrowser",
			"Enabled": true
		},
		{
			"Name": "WebSocketNetworking",
			"Enabled": true
		}
	]
}
